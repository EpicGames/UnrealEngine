// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class MeshMergeUtilities : ModuleRules
{
	public MeshMergeUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateDependencyModuleNames.AddRange(
			new string [] {
				"Core",
				"CoreUObject",
				"EditorFramework",
				"Engine",
				"RenderCore",
                "Renderer",
                "RHI",
                "Landscape",
                "UnrealEd",
                "MaterialUtilities",     
                "SlateCore",
                "Slate",
                "StaticMeshEditor",
                "MaterialBaking",
                "MeshUtilitiesCommon",
				"ToolMenus",
            }
		);
        
        PublicDependencyModuleNames.AddRange(
			new string [] {
                "RawMesh",
                "MeshDescription",
				"StaticMeshDescription",
            }
		);

        PublicIncludePathModuleNames.AddRange(
          new string[] {
               "MeshReductionInterface",
          }
        );
		
		PrivateIncludePathModuleNames.AddRange(
			new string [] {
				"HierarchicalLODUtilities"
			}
		);

        DynamicallyLoadedModuleNames.AddRange(
            new string[] {
                "HierarchicalLODUtilities",
                "MeshUtilities",
                "MeshReductionInterface",
            }
        );
    }
}
