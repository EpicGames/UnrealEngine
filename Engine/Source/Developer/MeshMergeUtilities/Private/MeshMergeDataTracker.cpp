// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshMergeDataTracker.h"
#include "MeshMergeHelpers.h"
#include "Misc/Crc.h"
#include "Engine/StaticMesh.h"
#include "StaticMeshAttributes.h"
#include "TriangleTypes.h"
#include "MaterialUtilities.h"

FMeshMergeDataTracker::FMeshMergeDataTracker()
	: AvailableLightMapUVChannel(INDEX_NONE), SummedLightMapPixels(0)
{
	FMemory::Memzero(bWithVertexColors);
	FMemory::Memzero(bOcuppiedUVChannels);
}

FMeshDescription& FMeshMergeDataTracker::AddAndRetrieveRawMesh(int32 MeshIndex, int32 LODIndex, UStaticMesh* InMesh)
{
	checkf(!RawMeshLODs.Contains(FMeshLODKey(MeshIndex, LODIndex, InMesh)), TEXT("Raw Mesh already added for this key"));
	FMeshDescription& MeshDescription = RawMeshLODs.Add(FMeshLODKey(MeshIndex, LODIndex, InMesh));
	FStaticMeshAttributes(MeshDescription).Register();
	return MeshDescription;
}

void FMeshMergeDataTracker::RemoveRawMesh(int32 MeshIndex, int32 LODIndex)
{
	checkf(RawMeshLODs.Contains(FMeshLODKey(MeshIndex, LODIndex)), TEXT("No Raw Mesh for this key"));
	RawMeshLODs.Remove(FMeshLODKey(MeshIndex, LODIndex));
}

TConstRawMeshIterator FMeshMergeDataTracker::GetConstRawMeshIterator() const
{
	return RawMeshLODs.CreateConstIterator();
}

TRawMeshIterator FMeshMergeDataTracker::GetRawMeshIterator()
{
	return RawMeshLODs.CreateIterator();
}

void FMeshMergeDataTracker::AddLightmapChannelRecord(int32 MeshIndex, int32 LODIndex, int32 LightmapChannelIndex)
{
	LightmapChannelLODs.Add(FMeshLODKey(MeshIndex, LODIndex), LightmapChannelIndex);
}

int32 FMeshMergeDataTracker::AddSection(const FSectionInfo& SectionInfo)
{
	return UniqueSections.AddUnique(SectionInfo);
}

int32 FMeshMergeDataTracker::NumberOfUniqueSections() const
{
	return UniqueSections.Num();
}

UMaterialInterface* FMeshMergeDataTracker::GetMaterialForSectionIndex(int32 SectionIndex)
{
	checkf(UniqueSections.IsValidIndex(SectionIndex), TEXT("Invalid section index for stored data"));
	return UniqueSections[SectionIndex].Material;
}

const FSectionInfo& FMeshMergeDataTracker::GetSection(int32 SectionIndex) const
{
	checkf(UniqueSections.IsValidIndex(SectionIndex), TEXT("Invalid section index for stored data"));
	return UniqueSections[SectionIndex];
}

void FMeshMergeDataTracker::AddBakedMaterialSection(const FSectionInfo& SectionInfo)
{
	UniqueSections.Empty(1);
	UniqueSections.AddUnique(SectionInfo);
}

void FMeshMergeDataTracker::AddMaterialSlotName(UMaterialInterface *MaterialInterface, FName MaterialSlotName)
{
	FName *FindMaterialSlotName = MaterialInterfaceToMaterialSlotName.Find(MaterialInterface);
	//If there is a material use by more then one slot, only the first slot name occurrence will be use. (selection order)
	if (FindMaterialSlotName == nullptr)
	{
		MaterialInterfaceToMaterialSlotName.Add(MaterialInterface, MaterialSlotName);
	}
}

FName FMeshMergeDataTracker::GetMaterialSlotName(UMaterialInterface *MaterialInterface) const
{
	const FName *MaterialSlotName = MaterialInterfaceToMaterialSlotName.Find(MaterialInterface);
	return MaterialSlotName == nullptr ? NAME_None : *MaterialSlotName;
}

void FMeshMergeDataTracker::AddLODIndex(int32 LODIndex)
{
	LODIndices.AddUnique(LODIndex);
}

int32 FMeshMergeDataTracker::GetNumLODsForMergedMesh() const
{
	return LODIndices.Num();
}

TConstLODIndexIterator FMeshMergeDataTracker::GetLODIndexIterator() const
{
	return LODIndices.CreateConstIterator();
}

void FMeshMergeDataTracker::AddLightMapPixels(int32 Dimension)
{
	SummedLightMapPixels += FMath::Max(Dimension, 0);
}

int32 FMeshMergeDataTracker::GetLightMapDimension() const
{
	return FMath::CeilToInt(FMath::Sqrt(static_cast<float>(SummedLightMapPixels)));
}

bool FMeshMergeDataTracker::DoesLODContainVertexColors(int32 LODIndex) const
{
	checkf(FMath::IsWithinInclusive(LODIndex, 0, MAX_STATIC_MESH_LODS - 1), TEXT("Invalid LOD index"));
	return bWithVertexColors[LODIndex];
}

bool FMeshMergeDataTracker::DoesAnyLODContainVertexColors() const
{
	for(int32 LODIndex = 0; LODIndex < MAX_STATIC_MESH_LODS; ++LODIndex)
	{
		if(bWithVertexColors[LODIndex])
		{
			return true;
		}
	}
	return false;
}

bool FMeshMergeDataTracker::DoesUVChannelContainData(int32 UVChannel, int32 LODIndex) const
{
	checkf(FMath::IsWithinInclusive(LODIndex, 0, MAX_STATIC_MESH_LODS - 1), TEXT("Invalid LOD index"));
	checkf(FMath::IsWithinInclusive(UVChannel, 0, MAX_MESH_TEXTURE_COORDS_MD - 1), TEXT("Invalid UV channel index"));
	return bOcuppiedUVChannels[LODIndex][UVChannel];
}

bool FMeshMergeDataTracker::DoesUVChannelContainData(int32 UVChannel) const
{
	checkf(FMath::IsWithinInclusive(UVChannel, 0, MAX_MESH_TEXTURE_COORDS_MD - 1), TEXT("Invalid UV channel index"));
	for(int32 LODIndex = 0; LODIndex < MAX_STATIC_MESH_LODS; LODIndex++)
	{
		if(bOcuppiedUVChannels[LODIndex][UVChannel])
		{
			return true;
		}
	}
	return false;
}

bool FMeshMergeDataTracker::DoesMeshLODRequireUniqueUVs(FMeshLODKey Key)
{
	// if we have vertex color, we require unique UVs
	return RequiresUniqueUVs.Contains(Key);
}

int32 FMeshMergeDataTracker::GetAvailableLightMapUVChannel() const
{
	return AvailableLightMapUVChannel;
}

void FMeshMergeDataTracker::AddComponentToWedgeMapping(int32 MeshIndex, int32 LODIndex, uint32 WedgeIndex)
{
	ComponentToWedgeOffsets.Add(FMeshLODKey(MeshIndex, LODIndex), WedgeIndex);
}

uint32 FMeshMergeDataTracker::GetComponentToWedgeMappng(int32 MeshIndex, int32 LODIndex) const
{
	const uint32* MappingPtr = ComponentToWedgeOffsets.Find(FMeshLODKey(MeshIndex, LODIndex));
	return MappingPtr ? *MappingPtr : INDEX_NONE;
}

FMeshDescription* FMeshMergeDataTracker::GetRawMeshPtr(int32 MeshIndex, int32 LODIndex)
{
	return RawMeshLODs.Find(FMeshLODKey(MeshIndex, LODIndex));
}

FMeshDescription* FMeshMergeDataTracker::GetRawMeshPtr(FMeshLODKey Key)
{
	return RawMeshLODs.Find(Key);
}

FMeshDescription* FMeshMergeDataTracker::FindRawMeshAndLODIndex(int32 MeshIndex, int32& OutLODIndex)
{
	FMeshDescription* FoundMeshPtr = nullptr;
	OutLODIndex = INDEX_NONE;
	for (TPair<FMeshLODKey, FMeshDescription>& Pair : RawMeshLODs)
	{
		if (Pair.Key.GetMeshIndex() == MeshIndex)
		{
			FoundMeshPtr = &Pair.Value;
			OutLODIndex = Pair.Key.GetLODIndex();
			break;
		}
	}

	return FoundMeshPtr;
}

FMeshDescription* FMeshMergeDataTracker::TryFindRawMeshForLOD(int32 MeshIndex, int32& InOutDesiredLODIndex)
{
	FMeshDescription* FoundMeshPtr = RawMeshLODs.Find(FMeshLODKey(MeshIndex, InOutDesiredLODIndex));
	int32 SearchIndex = InOutDesiredLODIndex - 1;
	while (FoundMeshPtr == nullptr && SearchIndex >= 0)
	{
		for (TPair<FMeshLODKey, FMeshDescription>& Pair : RawMeshLODs)
		{
			if (Pair.Key.GetMeshIndex() == MeshIndex && Pair.Key.GetLODIndex() == SearchIndex)
			{
				FoundMeshPtr = &Pair.Value;
				InOutDesiredLODIndex = SearchIndex;
				break;
			}
		}

		--SearchIndex;
	}

	return FoundMeshPtr;
}

void FMeshMergeDataTracker::AddSectionRemapping(int32 MeshIndex, int32 LODIndex, int32 OriginalIndex, int32 UniqueIndex)
{
	UniqueSectionIndexPerLOD.Add(FMeshLODKey(MeshIndex, LODIndex), SectionRemapPair(OriginalIndex, UniqueIndex));
	UniqueSectionToMeshLOD.Add(UniqueIndex, FMeshLODKey(MeshIndex, LODIndex));
}

void FMeshMergeDataTracker::GetMeshLODsMappedToUniqueSection(int32 UniqueIndex, TArray<FMeshLODKey>& InOutMeshLODs)
{
	UniqueSectionToMeshLOD.MultiFind(UniqueIndex, InOutMeshLODs);
}

void FMeshMergeDataTracker::GetMappingsForMeshLOD(FMeshLODKey Key, TArray<SectionRemapPair>& InOutMappings)
{
	UniqueSectionIndexPerLOD.MultiFind(Key, InOutMappings);
}

void FMeshMergeDataTracker::ProcessRawMeshes()
{
	bool bPotentialLightmapUVChannels[MAX_MESH_TEXTURE_COORDS_MD];
	FMemory::Memset(bPotentialLightmapUVChannels, 1);
	bool bPotentialLODLightmapUVChannels[MAX_STATIC_MESH_LODS][MAX_MESH_TEXTURE_COORDS_MD];
	FMemory::Memset(bPotentialLODLightmapUVChannels, 1);

	// Retrieve information in regards to occupied UV channels whether or not a mesh contains vertex colors, and if 
	for (TPair<FMeshLODKey, FMeshDescription>& MeshPair : RawMeshLODs)
	{
		FMeshLODKey& Key = MeshPair.Key;
		const int32 LODIndex = Key.GetLODIndex();
		const FMeshDescription& RawMesh = MeshPair.Value;

		FStaticMeshConstAttributes Attributes(RawMesh);
		TVertexInstanceAttributesConstRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
		TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

		// hash vertex color buffer so we can see if instances have unique vertex data
		if(VertexInstanceColors.GetNumElements() > 0)
		{
			Key.SetVertexColorHash(RawMesh.VertexInstanceAttributes().GetHash(MeshAttribute::VertexInstance::Color));
		}

		const int32 LightmapChannelIdx = LightmapChannelLODs.FindRef(Key);
		bool bNeedsVertexData = false;
		
		if (VertexInstanceUVs.GetNumElements() > 0)
		{
			for (int32 ChannelIndex = 0; ChannelIndex < FMath::Min(VertexInstanceUVs.GetNumChannels(), (int32)MAX_MESH_TEXTURE_COORDS_MD); ++ChannelIndex)
			{
				bOcuppiedUVChannels[LODIndex][ChannelIndex] = true;
				bPotentialLODLightmapUVChannels[LODIndex][ChannelIndex] = (ChannelIndex == LightmapChannelIdx);
				
				const bool bWrappingUVs = FMeshMergeHelpers::CheckWrappingUVs(RawMesh, ChannelIndex);
				if (bWrappingUVs)
				{
					bNeedsVertexData = true;
				}
			}
		}

		// Merge available lightmap slots from LODs into one set, so we can assess later what slots are available
		for (int32 ChannelIdx = 1; ChannelIdx < MAX_MESH_TEXTURE_COORDS_MD; ++ChannelIdx)
		{
			bPotentialLightmapUVChannels[ChannelIdx] &= bPotentialLODLightmapUVChannels[LODIndex][ChannelIdx];
		}

		if (bNeedsVertexData)
		{
			RequiresUniqueUVs.Add(Key);
		}

		bWithVertexColors[LODIndex] |= VertexInstanceColors.GetNumElements() != 0;
	}

	// Look for an available lightmap slot we can use in the merged set
	// We start at channel 1 as merged meshes always use texcoord 0 for their expected mapping channel, so we cant use it;
	AvailableLightMapUVChannel = INDEX_NONE;
	for (int32 ChannelIdx = 1; ChannelIdx < MAX_MESH_TEXTURE_COORDS_MD; ++ChannelIdx)
	{
		if(bPotentialLightmapUVChannels[ChannelIdx])
		{
			AvailableLightMapUVChannel = ChannelIdx;
			break;
		}
	}
}

double FMeshMergeDataTracker::GetTextureSizeFromTargetTexelDensity(float InTargetTexelDensity) const
{
	double Mesh3DArea = 0;
	const double MeshUVArea = 1.0;		// UVs are not available yet, assume perfect UV space usage.

	for (const TPair<FMeshLODKey, FMeshDescription>& MeshPair : RawMeshLODs)
	{
		const FMeshDescription& MeshDescription = MeshPair.Value;
			
		FStaticMeshConstAttributes Attributes(MeshDescription);
		TVertexAttributesConstRef<FVector3f> Positions = Attributes.GetVertexPositions();
		TUVAttributesConstRef<FVector2f> UVs = Attributes.GetUVCoordinates(0);

		for (const FTriangleID TriangleID : MeshDescription.Triangles().GetElementIDs())
		{
			// World space area
			TArrayView<const FVertexID> TriVertices = MeshDescription.GetTriangleVertices(TriangleID);
			Mesh3DArea += UE::Geometry::VectorUtil::Area(Positions[TriVertices[0]], Positions[TriVertices[1]], Positions[TriVertices[2]]);
		}
	}

	return FMaterialUtilities::GetTextureSizeFromTargetTexelDensity(Mesh3DArea, MeshUVArea, InTargetTexelDensity);
}