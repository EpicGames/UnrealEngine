// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshMergeHelpers.h"

#include "Engine/MapBuildDataRegistry.h"
#include "MeshMerge/MeshMergingSettings.h"
#include "MeshMerge/MeshProxySettings.h"

#include "MaterialOptions.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshOperations.h"

#include "Misc/PackageName.h"
#include "MaterialUtilities.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Rendering/SkeletalMeshModel.h"

#include "SkeletalMeshTypes.h"
#include "SkeletalRenderPublic.h"

#include "UObject/UObjectBaseUtility.h"
#include "UObject/Package.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "HierarchicalLODUtilitiesModule.h"
#include "MeshMergeData.h"
#include "IHierarchicalLODUtilities.h"
#include "Engine/MeshMergeCullingVolume.h"

#include "Landscape.h"
#include "LandscapeProxy.h"

#include "Editor.h"

#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/BodySetup.h"
#include "MeshUtilities.h"
#include "ImageUtils.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "IMeshReductionManagerModule.h"
#include "LayoutUV.h"
#include "Components/InstancedStaticMeshComponent.h"

//DECLARE_LOG_CATEGORY_CLASS(LogMeshMerging, Verbose, All);

static TAutoConsoleVariable<int32> CVarMeshMergeStoreImposterInfoInUVs(
	TEXT("r.MeshMerge.StoreImposterInfoInUVs"),
	0,
	TEXT("Determines whether or not to store imposter info (position.xy in UV2, position.z + scale in UV3) in the merged mesh UV channels\n")
	TEXT("0: Do not store imposters info in UVs (default)\n")
	TEXT("1: Store imposter info in UVs (legacy)\n"),
	ECVF_Default);

void FMeshMergeHelpers::ExtractSections(const UStaticMeshComponent* Component, int32 LODIndex, TArray<FSectionInfo>& OutSections)
{
	static UMaterialInterface* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface);

	const UStaticMesh* StaticMesh = Component->GetStaticMesh();

	TArray<FName> MaterialSlotNames;
	for (const FStaticMaterial& StaticMaterial : StaticMesh->GetStaticMaterials())
	{
#if WITH_EDITOR
		MaterialSlotNames.Add(StaticMaterial.ImportedMaterialSlotName);
#else
		MaterialSlotNames.Add(StaticMaterial.MaterialSlotName);
#endif
	}

	for (const FStaticMeshSection& MeshSection : StaticMesh->GetRenderData()->LODResources[LODIndex].Sections)
	{
		// Skip empty sections
		if (MeshSection.NumTriangles == 0)
		{
			continue;
		}

		// Retrieve material for this section
		UMaterialInterface* StoredMaterial = Component->GetMaterial(MeshSection.MaterialIndex);

		// Make sure the resource actual exists, otherwise use default material
		StoredMaterial = (StoredMaterial != nullptr) && StoredMaterial->GetMaterialResource(GMaxRHIShaderPlatform) ? StoredMaterial : DefaultMaterial;

		// Populate section data
		FSectionInfo SectionInfo;
		SectionInfo.Material = StoredMaterial;
		SectionInfo.MaterialIndex = MeshSection.MaterialIndex;
		SectionInfo.MaterialSlotName = MaterialSlotNames.IsValidIndex(MeshSection.MaterialIndex) ? MaterialSlotNames[MeshSection.MaterialIndex] : NAME_None;
		SectionInfo.StartIndex = MeshSection.FirstIndex / 3;
		SectionInfo.EndIndex = SectionInfo.StartIndex + MeshSection.NumTriangles;

		if (MeshSection.bEnableCollision)
		{
			SectionInfo.EnabledProperties.Add(GET_MEMBER_NAME_CHECKED(FStaticMeshSection, bEnableCollision));
		}

		if (MeshSection.bCastShadow && Component->CastShadow)
		{
			SectionInfo.EnabledProperties.Add(GET_MEMBER_NAME_CHECKED(FStaticMeshSection, bCastShadow));
		}

		OutSections.Add(SectionInfo);
	}
}

void FMeshMergeHelpers::ExtractSections(const USkeletalMeshComponent* Component, int32 LODIndex, TArray<FSectionInfo>& OutSections)
{
	static UMaterialInterface* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	FSkeletalMeshModel* Resource = Component->GetSkeletalMeshAsset()->GetImportedModel();

	checkf(Resource->LODModels.IsValidIndex(LODIndex), TEXT("Invalid LOD Index"));

	TArray<FName> MaterialSlotNames = Component->GetMaterialSlotNames();

	const FSkeletalMeshLODInfo* LODInfoPtr = Component->GetSkeletalMeshAsset()->GetLODInfo(LODIndex);
	check(LODInfoPtr);
	const FSkeletalMeshLODModel& Model = Resource->LODModels[LODIndex];
	for (int32 SectionIndex = 0; SectionIndex < Model.Sections.Num(); ++SectionIndex)
	{
		const FSkelMeshSection& MeshSection = Model.Sections[SectionIndex];
		// Retrieve material for this section
		int32 MaterialIndex = MeshSection.MaterialIndex;
		if (LODInfoPtr->LODMaterialMap.IsValidIndex(SectionIndex) && LODInfoPtr->LODMaterialMap[SectionIndex] != INDEX_NONE)
		{
			MaterialIndex = LODInfoPtr->LODMaterialMap[SectionIndex];
		}
		
		UMaterialInterface* StoredMaterial = Component->GetMaterial(MaterialIndex);
		// Make sure the resource actual exists, otherwise use default material
		StoredMaterial = (StoredMaterial != nullptr) && StoredMaterial->GetMaterialResource(GMaxRHIShaderPlatform) ? StoredMaterial : DefaultMaterial;

		FSectionInfo SectionInfo;
		SectionInfo.Material = StoredMaterial;
		SectionInfo.MaterialSlotName = MaterialSlotNames.IsValidIndex(MaterialIndex) ? MaterialSlotNames[MaterialIndex] : NAME_None;

		if (MeshSection.bCastShadow && Component->CastShadow)
		{
			SectionInfo.EnabledProperties.Add(GET_MEMBER_NAME_CHECKED(FSkelMeshSection, bCastShadow));
		}

		if (MeshSection.bVisibleInRayTracing)
		{
			SectionInfo.EnabledProperties.Add(GET_MEMBER_NAME_CHECKED(FSkelMeshSection, bVisibleInRayTracing));
		}

		if (MeshSection.bRecomputeTangent)
		{
			SectionInfo.EnabledProperties.Add(GET_MEMBER_NAME_CHECKED(FSkelMeshSection, bRecomputeTangent));
		}

		OutSections.Add(SectionInfo);
	}
}

void FMeshMergeHelpers::ExtractSections(const UStaticMesh* StaticMesh, int32 LODIndex, TArray<FSectionInfo>& OutSections)
{
	static UMaterialInterface* DefaultMaterial = UMaterial::GetDefaultMaterial(MD_Surface);

	for (const FStaticMeshSection& MeshSection : StaticMesh->GetRenderData()->LODResources[LODIndex].Sections)
	{
		// Retrieve material for this section
		UMaterialInterface* StoredMaterial = StaticMesh->GetMaterial(MeshSection.MaterialIndex);

		// Make sure the resource actual exists, otherwise use default material
		StoredMaterial = (StoredMaterial != nullptr) && StoredMaterial->GetMaterialResource(GMaxRHIShaderPlatform) ? StoredMaterial : DefaultMaterial;

		// Populate section data
		FSectionInfo SectionInfo;
		SectionInfo.Material = StoredMaterial;
		SectionInfo.MaterialIndex = MeshSection.MaterialIndex;
#if WITH_EDITOR
		SectionInfo.MaterialSlotName = StaticMesh->GetStaticMaterials().IsValidIndex(MeshSection.MaterialIndex) ? StaticMesh->GetStaticMaterials()[MeshSection.MaterialIndex].ImportedMaterialSlotName : NAME_None;
#else
		SectionInfo.MaterialSlotName = StaticMesh->GetStaticMaterials().IsValidIndex(MeshSection.MaterialIndex) ? StaticMesh->GetStaticMaterials()[MeshSection.MaterialIndex].MaterialSlotName : NAME_None;
#endif
		

		if (MeshSection.bEnableCollision)
		{
			SectionInfo.EnabledProperties.Add(GET_MEMBER_NAME_CHECKED(FStaticMeshSection, bEnableCollision));
		}

		if (MeshSection.bCastShadow)
		{
			SectionInfo.EnabledProperties.Add(GET_MEMBER_NAME_CHECKED(FStaticMeshSection, bCastShadow));
		}

		OutSections.Add(SectionInfo);
	}
}

void FMeshMergeHelpers::ExpandInstances(const UInstancedStaticMeshComponent* InInstancedStaticMeshComponent, FMeshDescription& InOutMeshDescription)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMergeHelpers::ExpandInstances)

	FMeshDescription CombinedMeshDescription;
	FStaticMeshAttributes(CombinedMeshDescription).Register();

	FMatrix ComponentToWorld = InInstancedStaticMeshComponent->GetComponentTransform().ToMatrixWithScale();
	FMatrix WorldToComponent = ComponentToWorld.Inverse();

	// Reserve memory upfront
	int32 NumInstances = InInstancedStaticMeshComponent->GetInstanceCount();

	CombinedMeshDescription.ReserveNewVertices(InOutMeshDescription.Vertices().Num() * NumInstances);
	CombinedMeshDescription.ReserveNewVertexInstances(InOutMeshDescription.VertexInstances().Num() * NumInstances);
	CombinedMeshDescription.ReserveNewEdges(InOutMeshDescription.Edges().Num() * NumInstances);
	CombinedMeshDescription.ReserveNewTriangles(InOutMeshDescription.Triangles().Num() * NumInstances);

	FStaticMeshOperations::FAppendSettings AppendSettings;
	for (int32 ChannelIdx = 0; ChannelIdx < FStaticMeshOperations::FAppendSettings::MAX_NUM_UV_CHANNELS; ++ChannelIdx)
	{
		AppendSettings.bMergeUVChannels[ChannelIdx] = true;
	}

	FMeshDescription InstanceMeshDescription;
	for (const FInstancedStaticMeshInstanceData& InstanceData : InInstancedStaticMeshComponent->PerInstanceSMData)
	{
		InstanceMeshDescription = InOutMeshDescription;
		FStaticMeshOperations::ApplyTransform(InstanceMeshDescription, WorldToComponent * InstanceData.Transform * ComponentToWorld);
		FStaticMeshOperations::AppendMeshDescription(InstanceMeshDescription, CombinedMeshDescription, AppendSettings);
	}

	InOutMeshDescription = CombinedMeshDescription;
}


static void RetrieveMeshInternal(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex, FMeshDescription& OutMeshDescription, bool bPropagateVertexColours, bool bApplyComponentTransform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMergeHelpers::RetrieveMeshInternal)

	check(StaticMesh);

	// Export the mesh data using static mesh render data
	StaticMesh->ExportStaticMeshLOD(StaticMesh->GetRenderData()->LODResources[LODIndex], OutMeshDescription);

	// Make sure the mesh is not irreparably malformed.
	if (OutMeshDescription.VertexInstances().Num() <= 0)
	{
		return;
	}

	// If we have a component, use it to retrieve transform & vertex colors (if requested)
	if (StaticMeshComponent)
	{
		// Handle spline mesh deformation
		const bool bIsSplineMeshComponent = StaticMeshComponent->IsA<USplineMeshComponent>();
		if (bIsSplineMeshComponent)
		{
			const USplineMeshComponent* SplineMeshComponent = Cast<USplineMeshComponent>(StaticMeshComponent);
			// Deform mesh data according to the Spline Mesh Component's data
			SplineMeshComponent->PropagateSplineDeformationToMesh(OutMeshDescription);
		}

		// If specified propagate painted vertex colors into our raw mesh
		if (bPropagateVertexColours)
		{
			FMeshMergeHelpers::PropagatePaintedColorsToMesh(StaticMeshComponent, LODIndex, OutMeshDescription);
		}

		// If specified transform vertices from local to world space
		if (bApplyComponentTransform)
		{
			FStaticMeshOperations::ApplyTransform(OutMeshDescription, StaticMeshComponent->GetComponentTransform());
		}
	}

	FMeshBuildSettings BuildSettings;
	
	// If editor data is not available, we won't have access to source model
	const bool bHasSourceModels = StaticMesh->IsSourceModelValid(0);
	if (bHasSourceModels)
	{ 
		// Imported meshes will have a valid mesh description
		const bool bImportedMesh = bHasSourceModels ? false : StaticMesh->IsMeshDescriptionValid(LODIndex);

		// Use build settings from base mesh for LOD entries that were generated inside Editor.
		BuildSettings = bImportedMesh ? StaticMesh->GetSourceModel(LODIndex).BuildSettings : StaticMesh->GetSourceModel(0).BuildSettings;
	}

	// Figure out if we should recompute normals and tangents. By default generated LODs should not recompute normals
	EComputeNTBsFlags ComputeNTBsOptions = EComputeNTBsFlags::BlendOverlappingNormals;
	if (BuildSettings.bRemoveDegenerates)
	{
		// If removing degenerate triangles, ignore them when computing tangents.
		ComputeNTBsOptions |= EComputeNTBsFlags::IgnoreDegenerateTriangles;
	}
	if (BuildSettings.bUseMikkTSpace)
	{
		ComputeNTBsOptions |= EComputeNTBsFlags::UseMikkTSpace;
	}

	FStaticMeshOperations::ComputeTriangleTangentsAndNormals(OutMeshDescription, 0.0f);
	FStaticMeshOperations::RecomputeNormalsAndTangentsIfNeeded(OutMeshDescription, ComputeNTBsOptions);
}

void FMeshMergeHelpers::RetrieveMesh(const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex, FMeshDescription& OutMeshDescription, bool bPropagateVertexColours, bool bApplyComponentTransform)
{
	RetrieveMeshInternal(StaticMeshComponent->GetStaticMesh(), StaticMeshComponent, LODIndex, OutMeshDescription, bPropagateVertexColours, bApplyComponentTransform);
}

void FMeshMergeHelpers::RetrieveMesh(const UStaticMesh* StaticMesh, int32 LODIndex, FMeshDescription& OutMeshDescription)
{
	RetrieveMeshInternal(StaticMesh, nullptr, LODIndex, OutMeshDescription, /*bPropagateVertexColours*/false, /*bApplyComponentTransform*/false);
}

void FMeshMergeHelpers::RetrieveMesh(const USkeletalMeshComponent* SkeletalMeshComponent, int32 LODIndex, FMeshDescription& OutMeshDescription, bool bPropagateVertexColours, bool bApplyComponentTransform)
{
	FSkeletalMeshModel* Resource = SkeletalMeshComponent->GetSkeletalMeshAsset()->GetImportedModel();
	if (Resource->LODModels.IsValidIndex(LODIndex))
	{
		FSkeletalMeshLODInfo& SrcLODInfo = *(SkeletalMeshComponent->GetSkeletalMeshAsset()->GetLODInfo(LODIndex));

		// Get the CPU skinned verts for this LOD
		TArray<FFinalSkinVertex> FinalVertices;

		// GetCPUSkinnedVertices 
		SkeletalMeshComponent->GetCPUSkinnedVertices(FinalVertices, LODIndex);

		FSkeletalMeshLODModel& LODModel = Resource->LODModels[LODIndex];

		const int32 NumSections = LODModel.Sections.Num();

		// Empty the mesh description
		OutMeshDescription.Empty();

		FStaticMeshAttributes Attributes(OutMeshDescription);
		TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
		TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
		TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
		TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
		TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
		TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
		TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
		TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

		int32 TotalTriangles = 0;
		int32 TotalCorners = 0;
		for (int32 SectionIndex = 0; SectionIndex < NumSections; SectionIndex++)
		{
			const FSkelMeshSection& SkelMeshSection = LODModel.Sections[SectionIndex];
			TotalTriangles += SkelMeshSection.NumTriangles;
		}
		TotalCorners = TotalTriangles * 3;
		OutMeshDescription.ReserveNewVertices(FinalVertices.Num());
		OutMeshDescription.ReserveNewPolygons(TotalTriangles);
		OutMeshDescription.ReserveNewVertexInstances(TotalCorners);
		OutMeshDescription.ReserveNewEdges(TotalCorners);

		// Copy skinned vertex positions
		for (int32 VertIndex = 0; VertIndex < FinalVertices.Num(); ++VertIndex)
		{
			const FVertexID VertexID = OutMeshDescription.CreateVertex();
			VertexPositions[VertexID] = FinalVertices[VertIndex].Position;
		}

		VertexInstanceUVs.SetNumChannels(MAX_TEXCOORDS);


		for (int32 SectionIndex = 0; SectionIndex < NumSections; SectionIndex++)
		{
			const FSkelMeshSection& SkelMeshSection = LODModel.Sections[SectionIndex];
			const int32 NumWedges = SkelMeshSection.NumTriangles * 3;

			//Create the polygon group ID
			int32 MaterialIndex = SkelMeshSection.MaterialIndex;
			// use the remapping of material indices if there is a valid value
			if (SrcLODInfo.LODMaterialMap.IsValidIndex(SectionIndex) && SrcLODInfo.LODMaterialMap[SectionIndex] != INDEX_NONE)
			{
				MaterialIndex = FMath::Clamp<int32>(SrcLODInfo.LODMaterialMap[SectionIndex], 0, SkeletalMeshComponent->GetSkeletalMeshAsset()->GetMaterials().Num() - 1);
			}

			FName ImportedMaterialSlotName = SkeletalMeshComponent->GetSkeletalMeshAsset()->GetMaterials()[MaterialIndex].ImportedMaterialSlotName;
			const FPolygonGroupID SectionPolygonGroupID(SectionIndex);
			if (!OutMeshDescription.IsPolygonGroupValid(SectionPolygonGroupID))
			{
				OutMeshDescription.CreatePolygonGroupWithID(SectionPolygonGroupID);
				PolygonGroupImportedMaterialSlotNames[SectionPolygonGroupID] = ImportedMaterialSlotName;
			}
			int32 WedgeIndex = 0;
			for (uint32 SectionTriangleIndex = 0; SectionTriangleIndex < SkelMeshSection.NumTriangles; ++SectionTriangleIndex)
			{
				FVertexID VertexIndexes[3];
				TArray<FVertexInstanceID> VertexInstanceIDs;
				VertexInstanceIDs.SetNum(3);
				for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex, ++WedgeIndex)
				{
					const int32 VertexIndexForWedge = LODModel.IndexBuffer[SkelMeshSection.BaseIndex + WedgeIndex];
					VertexIndexes[CornerIndex] = FVertexID(VertexIndexForWedge);
					FVertexInstanceID VertexInstanceID = OutMeshDescription.CreateVertexInstance(VertexIndexes[CornerIndex]);
					VertexInstanceIDs[CornerIndex] = VertexInstanceID;

					const FSoftSkinVertex& SoftVertex = SkelMeshSection.SoftVertices[VertexIndexForWedge - SkelMeshSection.BaseVertexIndex];
					const FFinalSkinVertex& SkinnedVertex = FinalVertices[VertexIndexForWedge];

					//Set NTBs
					const FVector TangentX = SkinnedVertex.TangentX.ToFVector();
					const FVector TangentZ = SkinnedVertex.TangentZ.ToFVector();
					//@todo: do we need to inverse the sign between skeletalmesh and staticmesh, the old code was doing so.
					const float TangentYSign = SkinnedVertex.TangentZ.ToFVector4f().W;

					VertexInstanceTangents[VertexInstanceID] = (FVector3f)TangentX;
					VertexInstanceBinormalSigns[VertexInstanceID] = TangentYSign;
					VertexInstanceNormals[VertexInstanceID] = (FVector3f)TangentZ;

					for (uint32 TexCoordIndex = 0; TexCoordIndex < MAX_TEXCOORDS; TexCoordIndex++)
					{
						//Add this vertex instance tex coord
						VertexInstanceUVs.Set(VertexInstanceID, TexCoordIndex, SoftVertex.UVs[TexCoordIndex]);
					}

					//Add this vertex instance color
					VertexInstanceColors[VertexInstanceID] = bPropagateVertexColours ? FVector4f(FLinearColor(SoftVertex.Color)) : FVector4f(1.0f, 1.0f, 1.0f);
				}
				//Create a polygon from this triangle
				const FPolygonID NewPolygonID = OutMeshDescription.CreatePolygon(SectionPolygonGroupID, VertexInstanceIDs);
			}
		}

		// If specified transform vertices from local to world space
		if (bApplyComponentTransform)
		{
			FStaticMeshOperations::ApplyTransform(OutMeshDescription, SkeletalMeshComponent->GetComponentTransform());
		}
	}
}


bool FMeshMergeHelpers::CheckWrappingUVs(const TArray<FVector2D>& UVs)
{	
	bool bResult = false;

	FVector2D Min(FLT_MAX, FLT_MAX);
	FVector2D Max(-FLT_MAX, -FLT_MAX);
	for (const FVector2D& Coordinate : UVs)
	{
		if ((Coordinate.X < 0.0f || Coordinate.Y < 0.0f) || (Coordinate.X > (1.0f + KINDA_SMALL_NUMBER) || Coordinate.Y > (1.0f + KINDA_SMALL_NUMBER)))
		{
			bResult = true;
			break;
		}
	}

	return bResult;
}

bool FMeshMergeHelpers::CheckWrappingUVs(const FMeshDescription& MeshDescription, int32 UVChannelIndex)
{
	TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = FStaticMeshConstAttributes(MeshDescription).GetVertexInstanceUVs();
	bool bResult = false;

	//Validate the channel, return false if there is an invalid channel index
	if (UVChannelIndex < 0 || UVChannelIndex >= VertexInstanceUVs.GetNumChannels())
	{
		return bResult;
	}

	for (const FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
	{
		const FVector2D& Coordinate = FVector2D(VertexInstanceUVs.Get(VertexInstanceID, UVChannelIndex));
		if ((Coordinate.X < 0.0f || Coordinate.Y < 0.0f) || (Coordinate.X > (1.0f + KINDA_SMALL_NUMBER) || Coordinate.Y > (1.0f + KINDA_SMALL_NUMBER)))
		{
			bResult = true;
			break;
		}
	}

	return bResult;
}

void FMeshMergeHelpers::CullTrianglesFromVolumesAndUnderLandscapes(const UWorld* World, const FBoxSphereBounds& Bounds, FMeshDescription& InOutMeshDescription)
{
	TArray<ALandscapeProxy*> Landscapes;
	TArray<AMeshMergeCullingVolume*> CullVolumes;

	FBox BoxBounds = Bounds.GetBox();

	for (ULevel* Level : World->GetLevels())
	{
		for (AActor* Actor : Level->Actors)
		{
			ALandscapeProxy* LandscapeProxy = Cast<ALandscapeProxy>(Actor);
			if (LandscapeProxy && LandscapeProxy->bUseLandscapeForCullingInvisibleHLODVertices)
			{
				FVector Origin, Extent;
				LandscapeProxy->GetActorBounds(false, Origin, Extent);
				FBox LandscapeBox(Origin - Extent, Origin + Extent);

				// Ignore Z axis for 2d bounds check
				if (LandscapeBox.IntersectXY(BoxBounds))
				{
					Landscapes.Add(LandscapeProxy);
				}
			}

			// Check for culling volumes
			AMeshMergeCullingVolume* Volume = Cast<AMeshMergeCullingVolume>(Actor);
			if (Volume)
			{
				// If the mesh's bounds intersect with the volume there is a possibility of culling
				const bool bIntersecting = Volume->EncompassesPoint(Bounds.Origin, Bounds.SphereRadius, nullptr);
				if (bIntersecting)
				{
					CullVolumes.Add(Volume);
				}
			}
		}
	}

	TVertexAttributesConstRef<FVector3f> VertexPositions = InOutMeshDescription.GetVertexPositions();

	TMap<FVertexID, bool> VertexVisible;
	VertexVisible.Reserve(InOutMeshDescription.Vertices().Num());
	int32 Index = 0;
	for(const FVertexID VertexID : InOutMeshDescription.Vertices().GetElementIDs())
	{
		const FVector3f& Position = VertexPositions[VertexID];
		// Start with setting visibility to true on all vertices
		VertexVisible.Add(VertexID, true);

		// Check if this vertex is culled due to being underneath a landscape
		if (Landscapes.Num() > 0)
		{
			bool bVertexWithinLandscapeBounds = false;

			for (ALandscapeProxy* Proxy : Landscapes)
			{
				FVector Origin, Extent;
				Proxy->GetActorBounds(false, Origin, Extent);
				FBox LandscapeBox(Origin - Extent, Origin + Extent);
				bVertexWithinLandscapeBounds |= LandscapeBox.IsInsideXY((FVector)Position);
			}

			if (bVertexWithinLandscapeBounds)
			{
				const FVector Start = (FVector)Position;
				FVector End = (FVector)Position - (WORLD_MAX * FVector::UpVector);
				FVector OutHit;
				const bool IsAboveLandscape = IsLandscapeHit(Start, End, World, Landscapes, OutHit);

				End = (FVector)Position + (WORLD_MAX * FVector::UpVector);
				const bool IsUnderneathLandscape = IsLandscapeHit(Start, End, World, Landscapes, OutHit);

				// Vertex is visible when above landscape (with actual landscape underneath) or if there is no landscape beneath or above the vertex (falls outside of landscape bounds)
				VertexVisible[VertexID] = (IsAboveLandscape && !IsUnderneathLandscape);// || (!IsAboveLandscape && !IsUnderneathLandscape);
			}
		}

		// Volume culling	
		for (AMeshMergeCullingVolume* Volume : CullVolumes)
		{
			const bool bVertexIsInsideVolume = Volume->EncompassesPoint((FVector)Position, 0.0f, nullptr);
			if (bVertexIsInsideVolume)
			{
				// Inside a culling volume so invisible
				VertexVisible[VertexID] = false;
			}
		}

		Index++;
	}


	// We now know which vertices are below the landscape
	TArray<FTriangleID> TriangleToRemove;
	for(const FTriangleID TriangleID : InOutMeshDescription.Triangles().GetElementIDs())
	{
		bool AboveLandscape = false;
		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			AboveLandscape |= VertexVisible[InOutMeshDescription.GetVertexInstanceVertex(InOutMeshDescription.GetTriangleVertexInstance(TriangleID, Corner))];
		}

		if (!AboveLandscape)
		{
			TriangleToRemove.Add(TriangleID);
		}
	}

	// Delete the polygons that are not visible
	{
		TArray<FEdgeID> OrphanedEdges;
		TArray<FVertexInstanceID> OrphanedVertexInstances;
		TArray<FPolygonGroupID> OrphanedPolygonGroups;
		TArray<FVertexID> OrphanedVertices;
		for (FTriangleID TriangleID : TriangleToRemove)
		{
			InOutMeshDescription.DeleteTriangle(TriangleID, &OrphanedEdges, &OrphanedVertexInstances, &OrphanedPolygonGroups);
		}
		//Do not remove the polygongroup since its indexed with the mesh material array
		/*for (FPolygonGroupID PolygonGroupID : OrphanedPolygonGroups)
		{
			InOutRawMesh.DeletePolygonGroup(PolygonGroupID);
		}*/
		for (FVertexInstanceID VertexInstanceID : OrphanedVertexInstances)
		{
			InOutMeshDescription.DeleteVertexInstance(VertexInstanceID, &OrphanedVertices);
		}
		for (FEdgeID EdgeID : OrphanedEdges)
		{
			InOutMeshDescription.DeleteEdge(EdgeID, &OrphanedVertices);
		}
		for (FVertexID VertexID : OrphanedVertices)
		{
			InOutMeshDescription.DeleteVertex(VertexID);
		}
		//Compact and Remap IDs so we have clean ID from 0 to n since we just erase some polygons
		//The render build need to have compact ID
		FElementIDRemappings OutRemappings;
		InOutMeshDescription.Compact(OutRemappings);
	}
}

void FMeshMergeHelpers::PropagateSplineDeformationToPhysicsGeometry(USplineMeshComponent* SplineMeshComponent, FKAggregateGeom& InOutPhysicsGeometry)
{
	const FVector Mask = USplineMeshComponent::GetAxisMask(SplineMeshComponent->GetForwardAxis());

	for (FKConvexElem& Elem : InOutPhysicsGeometry.ConvexElems)
	{
		for (FVector& Position : Elem.VertexData)
		{
			const float& AxisValue = USplineMeshComponent::GetAxisValueRef(Position, SplineMeshComponent->ForwardAxis);
			FTransform SliceTransform = SplineMeshComponent->CalcSliceTransform(AxisValue);
			Position = SliceTransform.TransformPosition(Position * Mask);
		}

		Elem.UpdateElemBox();
	}

	for (FKSphereElem& Elem : InOutPhysicsGeometry.SphereElems)
	{
		const FVector WorldSpaceCenter = Elem.GetTransform().TransformPosition(Elem.Center);
		Elem.Center = SplineMeshComponent->CalcSliceTransform(USplineMeshComponent::GetAxisValueRef(WorldSpaceCenter, SplineMeshComponent->ForwardAxis)).TransformPosition(Elem.Center * Mask);
	}

	for (FKSphylElem& Elem : InOutPhysicsGeometry.SphylElems)
	{
		const FVector WorldSpaceCenter = Elem.GetTransform().TransformPosition(Elem.Center);
		Elem.Center = SplineMeshComponent->CalcSliceTransform(USplineMeshComponent::GetAxisValueRef(WorldSpaceCenter, SplineMeshComponent->ForwardAxis)).TransformPosition(Elem.Center * Mask);
	}
}

void FMeshMergeHelpers::RetrieveCullingLandscapeAndVolumes(UWorld* InWorld, const FBoxSphereBounds& EstimatedMeshProxyBounds, const TEnumAsByte<ELandscapeCullingPrecision::Type> PrecisionType, TArray<FMeshDescription*>& OutCullingMeshes)
{
	// Extract landscape proxies and cull volumes from the world
	TArray<ALandscapeProxy*> LandscapeActors;
	TArray<AMeshMergeCullingVolume*> CullVolumes;

	uint32 MaxLandscapeExportLOD = 0;
	if (InWorld->IsValidLowLevel())
	{
		for (FConstLevelIterator Iterator = InWorld->GetLevelIterator(); Iterator; ++Iterator)
		{
			for (AActor* Actor : (*Iterator)->Actors)
			{
				if (Actor)
				{
					ALandscapeProxy* LandscapeProxy = Cast<ALandscapeProxy>(Actor);
					if (LandscapeProxy && LandscapeProxy->bUseLandscapeForCullingInvisibleHLODVertices)
					{
						// Retrieve highest landscape LOD level possible
						MaxLandscapeExportLOD = FMath::Max(MaxLandscapeExportLOD, FMath::CeilLogTwo(LandscapeProxy->SubsectionSizeQuads + 1) - 1);
						LandscapeActors.Add(LandscapeProxy);
					}
					// Check for culling volumes
					AMeshMergeCullingVolume* Volume = Cast<AMeshMergeCullingVolume>(Actor);
					if (Volume)
					{
						// If the mesh's bounds intersect with the volume there is a possibility of culling
						const bool bIntersecting = Volume->EncompassesPoint(EstimatedMeshProxyBounds.Origin, EstimatedMeshProxyBounds.SphereRadius, nullptr);
						if (bIntersecting)
						{
							CullVolumes.Add(Volume);
						}
					}
				}
			}
		}
	}

	// Setting determines the precision at which we should export the landscape for culling (highest, half or lowest)
	const uint32 LandscapeExportLOD = ((float)MaxLandscapeExportLOD * (0.5f * (float)PrecisionType));
	for (ALandscapeProxy* Landscape : LandscapeActors)
	{
		// Export the landscape to raw mesh format
		FMeshDescription* MeshDescription = new FMeshDescription();
		FStaticMeshAttributes(*MeshDescription).Register();
		ALandscapeProxy::FRawMeshExportParams ExportParams;
		ExportParams.ExportLOD = LandscapeExportLOD;
		ExportParams.ExportBounds = EstimatedMeshProxyBounds;
		Landscape->ExportToRawMesh(ExportParams, *MeshDescription);
		if (MeshDescription->Vertices().Num())
		{
			OutCullingMeshes.Add(MeshDescription);
		}
	}

	// Also add volume mesh data as culling meshes
	for (AMeshMergeCullingVolume* Volume : CullVolumes)
	{
		// Export the landscape to raw mesh format
		FMeshDescription* VolumeMesh = new FMeshDescription();
		FStaticMeshAttributes MeshAttributes(*VolumeMesh);
		MeshAttributes.Register();

		TArray<FStaticMaterial>	VolumeMaterials;
		GetBrushMesh(Volume, Volume->Brush, *VolumeMesh, VolumeMaterials);

		// Offset vertices to correct world position;
		FVector VolumeLocation = Volume->GetActorLocation();
		TVertexAttributesRef<FVector3f> VertexPositions = MeshAttributes.GetVertexPositions();
		for(const FVertexID VertexID : VolumeMesh->Vertices().GetElementIDs())
		{
			VertexPositions[VertexID] += (FVector3f)VolumeLocation;
		}

		OutCullingMeshes.Add(VolumeMesh);
	}
}

void FMeshMergeHelpers::TransformPhysicsGeometry(const FTransform& InTransform, const bool bBakeConvexTransform, struct FKAggregateGeom& AggGeom)
{
	FTransform NoScaleInTransform = InTransform;
	NoScaleInTransform.SetScale3D(FVector(1, 1, 1));

	// Pre-scale all non-convex geometry		
	const FVector Scale3D = InTransform.GetScale3D();
	if (!Scale3D.Equals(FVector(1.f)))
	{
		const float MinPrimSize = KINDA_SMALL_NUMBER;

		for (FKSphereElem& Elem : AggGeom.SphereElems)
		{
			Elem = Elem.GetFinalScaled(Scale3D, FTransform::Identity);
		}

		for (FKBoxElem& Elem : AggGeom.BoxElems)
		{
			Elem = Elem.GetFinalScaled(Scale3D, FTransform::Identity);
		}

		for (FKSphylElem& Elem : AggGeom.SphylElems)
		{
			Elem = Elem.GetFinalScaled(Scale3D, FTransform::Identity);
		}
	}
	
	// Multiply out merge transform (excluding scale) with original transforms for non-convex geometry
	for (FKSphereElem& Elem : AggGeom.SphereElems)
	{
		FTransform ElemTM = Elem.GetTransform();
		Elem.SetTransform(ElemTM*NoScaleInTransform);
	}

	for (FKBoxElem& Elem : AggGeom.BoxElems)
	{
		FTransform ElemTM = Elem.GetTransform();
		Elem.SetTransform(ElemTM*NoScaleInTransform);
	}

	for (FKSphylElem& Elem : AggGeom.SphylElems)
	{
		FTransform ElemTM = Elem.GetTransform();
		Elem.SetTransform(ElemTM*NoScaleInTransform);
	}

	for (FKConvexElem& Elem : AggGeom.ConvexElems)
	{
		FTransform ElemTM = Elem.GetTransform();
        if (bBakeConvexTransform)
        {
            for (FVector& Position : Elem.VertexData)
            {
                Position = ElemTM.TransformPosition(Position);
            }
		    Elem.SetTransform(InTransform);
        }
        else
        {
            Elem.SetTransform(ElemTM*InTransform);
        }
	}
}

void FMeshMergeHelpers::ExtractPhysicsGeometry(UBodySetup* InBodySetup, const FTransform& ComponentToWorld, const bool bBakeConvexTransform, struct FKAggregateGeom& OutAggGeom)
{
	if (InBodySetup == nullptr)
	{
		return;
	}


	OutAggGeom = InBodySetup->AggGeom;

	// Convert boxes to convex, so they can be sheared 
	for (int32 BoxIdx = 0; BoxIdx < OutAggGeom.BoxElems.Num(); BoxIdx++)
	{
		FKConvexElem* NewConvexColl = new(OutAggGeom.ConvexElems) FKConvexElem();
		NewConvexColl->ConvexFromBoxElem(OutAggGeom.BoxElems[BoxIdx]);
	}
	OutAggGeom.BoxElems.Empty();

	// we are not owner of this stuff
	OutAggGeom.FreeRenderInfo();
	for (FKConvexElem& Elem : OutAggGeom.ConvexElems)
	{
		Elem.ResetChaosConvexMesh();
	}

	// Transform geometry to world space
	TransformPhysicsGeometry(ComponentToWorld, bBakeConvexTransform, OutAggGeom);
}

FVector2D FMeshMergeHelpers::GetValidUV(const FVector2D& UV)
{
	FVector2D NewUV = UV;
	// first make sure they're positive
	if (UV.X < 0.0f)
	{
		NewUV.X = UV.X + FMath::CeilToInt(FMath::Abs(UV.X));
	}

	if (UV.Y < 0.0f)
	{
		NewUV.Y = UV.Y + FMath::CeilToInt(FMath::Abs(UV.Y));
	}

	// now make sure they're within [0, 1]
	if (UV.X > 1.0f)
	{
		NewUV.X = FMath::Fmod(NewUV.X, 1.0f);
	}

	if (UV.Y > 1.0f)
	{
		NewUV.Y = FMath::Fmod(NewUV.Y, 1.0f);
	}

	return NewUV;
}

void FMeshMergeHelpers::CalculateTextureCoordinateBoundsForMesh(const FMeshDescription& InMeshDescription, TArray<FBox2D>& OutBounds)
{
	TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = FStaticMeshConstAttributes(InMeshDescription).GetVertexInstanceUVs();
	OutBounds.Empty();
	for (const FTriangleID TriangleID : InMeshDescription.Polygons().GetElementIDs())
	{
		int32 MaterialIndex = InMeshDescription.GetTrianglePolygonGroup(TriangleID).GetValue();
		if (OutBounds.Num() <= MaterialIndex)
			OutBounds.SetNumZeroed(MaterialIndex + 1);
		{
			TArrayView<const FVertexInstanceID> TriangleVertexInstances = InMeshDescription.GetTriangleVertexInstances(TriangleID);
			for (const FVertexInstanceID VertexInstanceID : TriangleVertexInstances)
			{
				for (int32 UVIndex = 0; UVIndex < VertexInstanceUVs.GetNumChannels(); ++UVIndex)
				{
					OutBounds[MaterialIndex] += FVector2D(VertexInstanceUVs.Get(VertexInstanceID, UVIndex));
				}
			}
		}
	}
}

bool FMeshMergeHelpers::PropagatePaintedColorsToMesh(const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex, FMeshDescription& InOutMeshDescription)
{
	if (StaticMeshComponent->LODData.IsValidIndex(LODIndex) &&
		StaticMeshComponent->LODData[LODIndex].OverrideVertexColors != nullptr)
	{
		FColorVertexBuffer& ColorVertexBuffer = *StaticMeshComponent->LODData[LODIndex].OverrideVertexColors;
		const UStaticMesh& StaticMesh = *StaticMeshComponent->GetStaticMesh();

		if (ColorVertexBuffer.GetNumVertices() == StaticMesh.GetNumVertices(LODIndex))
		{	
			const FStaticMeshLODResources& RenderModel = StaticMesh.GetRenderData()->LODResources[LODIndex];

			const int32 NumWedges = InOutMeshDescription.VertexInstances().Num();
			const int32 NumRenderWedges = RenderModel.IndexBuffer.GetNumIndices();
			const bool bUseRenderWedges = NumWedges == NumRenderWedges;

			TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = FStaticMeshAttributes(InOutMeshDescription).GetVertexInstanceColors();

			if (bUseRenderWedges)
			{
				//Create a map index
				TMap<int32, FVertexInstanceID> IndexToVertexInstanceID;
				IndexToVertexInstanceID.Reserve(NumWedges);
				int32 CurrentWedgeIndex = 0;
				for (const FTriangleID TriangleID : InOutMeshDescription.Triangles().GetElementIDs())
				{
					for (int32 Corner = 0; Corner < 3; ++Corner, ++CurrentWedgeIndex)
					{
						IndexToVertexInstanceID.Add(CurrentWedgeIndex, InOutMeshDescription.GetTriangleVertexInstance(TriangleID, Corner));
					}
				}

				const FIndexArrayView ArrayView = RenderModel.IndexBuffer.GetArrayView();
				for (int32 WedgeIndex = 0; WedgeIndex < NumRenderWedges; WedgeIndex++)
				{
					const int32 Index = ArrayView[WedgeIndex];
					FColor WedgeColor = FColor::White;
					if (Index != INDEX_NONE)
					{
						WedgeColor = ColorVertexBuffer.VertexColor(Index);
					}
					VertexInstanceColors[IndexToVertexInstanceID[WedgeIndex]] = FLinearColor(WedgeColor);
				}

				return true;				
			}
			// No wedge map (this can happen when we poly reduce the LOD for example)
			// Use index buffer directly. Not sure this will happen with FMeshDescription
			else
			{
				if (InOutMeshDescription.Vertices().Num() == ColorVertexBuffer.GetNumVertices())
				{
					//Create a map index
					TMap<FVertexID, int32> VertexIDToVertexIndex;
					VertexIDToVertexIndex.Reserve(InOutMeshDescription.Vertices().Num());
					int32 CurrentVertexIndex = 0;
					for (const FVertexID VertexID : InOutMeshDescription.Vertices().GetElementIDs())
					{
						VertexIDToVertexIndex.Add(VertexID, CurrentVertexIndex++);
					}

					for (const FVertexID VertexID : InOutMeshDescription.Vertices().GetElementIDs())
					{
						FColor WedgeColor = FColor::White;
						uint32 VertIndex = VertexIDToVertexIndex[VertexID];

						if (VertIndex < ColorVertexBuffer.GetNumVertices())
						{
							WedgeColor = ColorVertexBuffer.VertexColor(VertIndex);
						}
						TArrayView<const FVertexInstanceID> VertexInstances = InOutMeshDescription.GetVertexVertexInstanceIDs(VertexID);
						for (const FVertexInstanceID& VertexInstanceID : VertexInstances)
						{
							VertexInstanceColors[VertexInstanceID] = FLinearColor(WedgeColor);
						}
					}
					return true;
				}
			}
		}
	}

	return false;
}

bool FMeshMergeHelpers::IsLandscapeHit(const FVector& RayOrigin, const FVector& RayEndPoint, const UWorld* World, const TArray<ALandscapeProxy*>& LandscapeProxies, FVector& OutHitLocation)
{
	TArray<FHitResult> Results;
	// Each landscape component has 2 collision shapes, 1 of them is specific to landscape editor
	// Trace only ECC_Visibility channel, so we do hit only Editor specific shape
	World->LineTraceMultiByObjectType(Results, RayOrigin, RayEndPoint, FCollisionObjectQueryParams(ECollisionChannel::ECC_WorldStatic), FCollisionQueryParams(SCENE_QUERY_STAT(LandscapeTrace), true));

	bool bHitLandscape = false;

	for (const FHitResult& HitResult : Results)
	{
		ULandscapeHeightfieldCollisionComponent* CollisionComponent = Cast<ULandscapeHeightfieldCollisionComponent>(HitResult.Component.Get());
		if (CollisionComponent)
		{
			ALandscapeProxy* HitLandscape = CollisionComponent->GetLandscapeProxy();
			if (HitLandscape && LandscapeProxies.Contains(HitLandscape))
			{
				// Could write a correct clipping algorithm, that clips the triangle to hit location
				OutHitLocation = HitLandscape->LandscapeActorToWorld().InverseTransformPosition(HitResult.Location);
				// Above landscape so visible
				bHitLandscape = true;
			}
		}
	}

	return bHitLandscape;
}

void FMeshMergeHelpers::MergeImpostersToMesh(TArray<const UStaticMeshComponent*> ImposterComponents, FMeshDescription& InMeshDescription, const FVector& InPivot, int32 InBaseMaterialIndex, TArray<UMaterialInterface*>& OutImposterMaterials)
{
	TMap<UMaterialInterface*, FPolygonGroupID> ImposterMaterialToPolygonGroupID;
	for (const UStaticMeshComponent* Component : ImposterComponents)
	{
		// Retrieve imposter LOD mesh and material			
		const int32 LODIndex = Component->GetStaticMesh()->GetNumLODs() - 1;

		// Retrieve mesh data in FMeshDescription form
		FMeshDescription ImposterMesh;
		FStaticMeshAttributes ImposterMeshAttributes(ImposterMesh);
		ImposterMeshAttributes.Register();
		FMeshMergeHelpers::RetrieveMesh(Component, LODIndex, ImposterMesh, false);

		// Retrieve the sections, we're expect 1 for imposter meshes
		TArray<FSectionInfo> Sections;
		FMeshMergeHelpers::ExtractSections(Component, LODIndex, Sections);

		TArray<int32> SectionImposterUniqueMaterialIndex;
		for (FSectionInfo& Info : Sections)
		{
			SectionImposterUniqueMaterialIndex.Add(OutImposterMaterials.AddUnique(Info.Material));
		}

		if (CVarMeshMergeStoreImposterInfoInUVs.GetValueOnAnyThread())
		{
			// Imposter magic, we're storing the actor world position and X scale spread across two UV channels
			const int32 UVOneIndex = 2;
			const int32 UVTwoIndex = UVOneIndex + 1;
			TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = ImposterMeshAttributes.GetVertexInstanceUVs();
			VertexInstanceUVs.SetNumChannels(UVTwoIndex + 1);
			const int32 NumIndices = ImposterMesh.VertexInstances().Num();
			const FTransform& ComponentToWorld = Component->GetComponentTransform();
			const FVector ComponentPosition = ComponentToWorld.TransformPosition(FVector::ZeroVector) - InPivot;
			for (const FVertexInstanceID VertexInstanceID : ImposterMesh.VertexInstances().GetElementIDs())
			{
				FVector2D UVOne;
				FVector2D UVTwo;

				UVOne.X = ComponentPosition.X;
				UVOne.Y = ComponentPosition.Y;
				VertexInstanceUVs.Set(VertexInstanceID, UVOneIndex, FVector2f(UVOne));	// LWC_TODO: Precision loss

				UVTwo.X = ComponentPosition.Z;
				UVTwo.Y = FMath::Abs(ComponentToWorld.GetScale3D().X);
				VertexInstanceUVs.Set(VertexInstanceID, UVTwoIndex, FVector2f(UVTwo));	// LWC_TODO: Precision loss
			}
		}
		else if (!InPivot.IsZero())
		{
			// Apply pivot offset if non null
			TVertexAttributesRef<FVector3f> ImposterMeshVertexPositions = ImposterMesh.GetVertexPositions();
			for (FVertexID VertexID : ImposterMesh.Vertices().GetElementIDs())
			{
				ImposterMeshVertexPositions[VertexID] -= (FVector3f)InPivot;
			}
		}

		TPolygonGroupAttributesRef<FName> SourcePolygonGroupImportedMaterialSlotNames = ImposterMeshAttributes.GetPolygonGroupMaterialSlotNames();

		FStaticMeshAttributes Attributes(InMeshDescription);
		TPolygonGroupAttributesRef<FName> TargetPolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();

		FStaticMeshOperations::FAppendSettings AppendSettings;

		//Add the missing polygon group ID to the target(InMeshDescription)
		//Remap the source mesh(ImposterMesh) polygongroup to fit with the target polygon groups
		AppendSettings.PolygonGroupsDelegate = FAppendPolygonGroupsDelegate::CreateLambda([&](const FMeshDescription& SourceMesh, FMeshDescription& TargetMesh, PolygonGroupMap& RemapPolygonGroups)
		{
			RemapPolygonGroups.Reserve(SourceMesh.PolygonGroups().Num());
			int32 SectionIndex = 0;
			for (const FPolygonGroupID SourcePolygonGroupID : SourceMesh.PolygonGroups().GetElementIDs())
			{
				UMaterialInterface* MaterialUseBySection = OutImposterMaterials[SectionImposterUniqueMaterialIndex[SectionIndex++]];
				FPolygonGroupID* ExistTargetPolygonGroupID = ImposterMaterialToPolygonGroupID.Find(MaterialUseBySection);
				FPolygonGroupID MatchTargetPolygonGroupID = ExistTargetPolygonGroupID == nullptr ? INDEX_NONE : *ExistTargetPolygonGroupID;
				if (MatchTargetPolygonGroupID == INDEX_NONE)
				{
					MatchTargetPolygonGroupID = TargetMesh.CreatePolygonGroup();
					//use the material name to fill the imported material name. Material name will be unique
					TargetPolygonGroupImportedMaterialSlotNames[MatchTargetPolygonGroupID] = MaterialUseBySection->GetFName();
					ImposterMaterialToPolygonGroupID.Add(MaterialUseBySection, MatchTargetPolygonGroupID);
				}
				RemapPolygonGroups.Add(SourcePolygonGroupID, MatchTargetPolygonGroupID);
			}
		});

		AppendSettings.bMergeVertexColor = true;
		for (int32 ChannelIdx = 0; ChannelIdx < FStaticMeshOperations::FAppendSettings::MAX_NUM_UV_CHANNELS; ++ChannelIdx)
		{
			AppendSettings.bMergeUVChannels[ChannelIdx] = true;
		}
		FStaticMeshOperations::AppendMeshDescription(ImposterMesh, InMeshDescription, AppendSettings);
	}
}

void FMeshMergeHelpers::FixupNonStandaloneMaterialReferences(UStaticMesh* InStaticMesh)
{
	UPackage* Package = InStaticMesh->GetPackage();

	// Ensure mesh is not referencing non standalone materials
	// This can be the case for MaterialInstanceDynamic (MID) as they are normally outered to the components
	for (FStaticMaterial& StaticMaterial : InStaticMesh->GetStaticMaterials())
	{
		TObjectPtr<UMaterialInterface>& MaterialInterface = StaticMaterial.MaterialInterface;
		if (MaterialInterface && !MaterialInterface->HasAnyFlags(RF_Standalone) && MaterialInterface->GetPackage() != Package)
		{
			// Duplicate material and outer it to the mesh
			MaterialInterface = DuplicateObject<UMaterialInterface>(MaterialInterface, InStaticMesh);
		}
	}
}
