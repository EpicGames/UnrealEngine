// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshMerge/MeshProxySettings.h"

#define UE_API MESHMERGEUTILITIES_API


class USkeletalMeshComponent;
class UStaticMeshComponent;
class UStaticMesh;
class USplineMeshComponent;
class UBodySetup;
class ALandscapeProxy;
struct FSectionInfo;
struct FMeshDescription;
struct FStaticMaterial;
struct FRawMeshExt;
struct FStaticMeshLODResources;
struct FKAggregateGeom;
class UInstancedStaticMeshComponent;

class FMeshMergeHelpers
{
public:
	/** Extracting section info data from static, skeletal mesh (components) */
	static UE_API void ExtractSections(const UStaticMeshComponent* Component, int32 LODIndex, TArray<FSectionInfo>& OutSections);
	static UE_API void ExtractSections(const USkeletalMeshComponent* Component, int32 LODIndex, TArray<FSectionInfo>& OutSections);
	static UE_API void ExtractSections(const UStaticMesh* StaticMesh, int32 LODIndex, TArray<FSectionInfo>& OutSections);

	/** Expanding instance data from instanced static mesh components */
	static UE_API void ExpandInstances(const UInstancedStaticMeshComponent* InInstancedStaticMeshComponent, FMeshDescription& InOutRawMesh);

	/** Extracting mesh data in FMeshDescription form from static, skeletal mesh (components) */
	static UE_API void RetrieveMesh(const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex, FMeshDescription& OutMeshDescription, bool bPropagateVertexColours, bool bApplyComponentTransform = true);
	static UE_API void RetrieveMesh(const USkeletalMeshComponent* SkeletalMeshComponent, int32 LODIndex, FMeshDescription& OutMeshDescription, bool bPropagateVertexColours, bool bApplyComponentTransform = false);
	static UE_API void RetrieveMesh(const UStaticMesh* StaticMesh, int32 LODIndex, FMeshDescription& OutMeshDescription);

	/** Checks whether or not the texture coordinates are outside of 0-1 UV ranges */
	static UE_API bool CheckWrappingUVs(const TArray<FVector2D>& UVs);
	static UE_API bool CheckWrappingUVs(const FMeshDescription& MeshDescription, int32 UVChannelIndex);

	/** Culls away triangles which are inside culling volumes or completely underneath the landscape */
	static UE_API void CullTrianglesFromVolumesAndUnderLandscapes(const UWorld* World, const FBoxSphereBounds& Bounds, FMeshDescription& InOutRawMesh);
	
	/** Propagates deformation along spline to physics geometry data */
	static UE_API void PropagateSplineDeformationToPhysicsGeometry(USplineMeshComponent* SplineMeshComponent, FKAggregateGeom& InOutPhysicsGeometry);

	/** Retrieves all culling landscapes and volumes as FMeshDescription structures. Note the caller is responsible for deleting the heap data managed by OutCullingMeshes */
	static UE_API void RetrieveCullingLandscapeAndVolumes(UWorld* InWorld, const FBoxSphereBounds& EstimatedMeshProxyBounds, const TEnumAsByte<ELandscapeCullingPrecision::Type> PrecisionType, TArray<FMeshDescription*>& OutCullingMeshes);

	/** Transforms physics geometry data using InTransform */
	static UE_API void TransformPhysicsGeometry(const FTransform& InTransform, const bool bBakeConvexTransform, struct FKAggregateGeom& AggGeom);
	
	/** Extract physics geometry data from a body setup */
	static UE_API void ExtractPhysicsGeometry(UBodySetup* InBodySetup, const FTransform& ComponentToWorld, const bool bBakeConvexTransform, struct FKAggregateGeom& OutAggGeom);

	/** Ensure that UV is in valid 0-1 UV ranges */
	static UE_API FVector2D GetValidUV(const FVector2D& UV);
	
	/** Calculates UV coordinates bounds for the given MeshDescription	*/
	static UE_API void CalculateTextureCoordinateBoundsForMesh(const FMeshDescription& InMeshDescription, TArray<FBox2D>& OutBounds);

	/** Propagates vertex painted colors from the StaticMeshComponent instance to MeshDescription */
	static UE_API bool PropagatePaintedColorsToMesh(const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex, FMeshDescription& InOutMeshDescription);

	/** Checks whether or not the landscape proxy is hit given a ray start and end */
	static UE_API bool IsLandscapeHit(const FVector& RayOrigin, const FVector& RayEndPoint, const UWorld* World, const TArray<ALandscapeProxy*>& LandscapeProxies, FVector& OutHitLocation);
	
	/** Merges imposter meshes into the given MeshDescription. */
	static UE_API void MergeImpostersToMesh(TArray<const UStaticMeshComponent*> ImposterComponents, FMeshDescription& InOutMeshDescription, const FVector& InPivot, int32 BaseMaterialIndex, TArray<UMaterialInterface*>& OutImposterMaterials);

	/** Ensure a generated HLOD mesh is not referencing non standalone materials. */
	static UE_API void FixupNonStandaloneMaterialReferences(UStaticMesh* InStaticMesh);
};

#undef UE_API
