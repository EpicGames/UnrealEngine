// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshMergeModule.h"
#include "MeshMergeUtilities.h"
#include "Modules/ModuleManager.h"
#include "ToolMenus.h"

class FMeshMergeModule : public IMeshMergeModule
{
public:
	virtual const IMeshMergeUtilities& GetUtilities() const override
	{
		return *dynamic_cast<const IMeshMergeUtilities*>(&Utilities);
	}

	virtual IMeshMergeUtilities& GetUtilities() override
	{
		return Utilities;
	}

	virtual void StartupModule() override
	{
		FModuleManager::Get().LoadModuleChecked("StaticMeshDescription");
	}

	virtual void ShutdownModule() override
	{
		UToolMenus::UnregisterOwner(this);
	}

protected:
	FMeshMergeUtilities Utilities;

};


IMPLEMENT_MODULE(FMeshMergeModule, MeshMergeUtilities);