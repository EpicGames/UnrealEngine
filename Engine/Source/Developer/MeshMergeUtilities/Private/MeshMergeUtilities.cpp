// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshMergeUtilities.h"

#include "Engine/MapBuildDataRegistry.h"
#include "MeshMerge/MeshInstancingSettings.h"
#include "MeshMerge/MeshMergingSettings.h"
#include "MeshMerge/MeshProxySettings.h"
#include "Engine/StaticMeshSocket.h"

#include "MaterialOptions.h"
#include "IMaterialBakingModule.h"

#include "Misc/PackageName.h"
#include "MaterialUtilities.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Components/ShapeComponent.h"

#include "SkeletalMeshTypes.h"
#include "SkeletalRenderPublic.h"

#include "UObject/UObjectBaseUtility.h"
#include "UObject/Package.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "HierarchicalLODUtilitiesModule.h"
#include "MeshMergeData.h"
#include "IHierarchicalLODUtilities.h"
#include "Engine/MeshMergeCullingVolume.h"

#include "Landscape.h"
#include "LandscapeProxy.h"

#include "Editor.h"
#include "ProxyGenerationProcessor.h"
#include "Editor/EditorPerProjectUserSettings.h"

#include "Engine/StaticMesh.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/BodySetup.h"
#include "MeshUtilities.h"
#include "ImageUtils.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "IMeshReductionManagerModule.h"
#include "IMeshReductionInterfaces.h"

#include "ProxyGenerationProcessor.h"
#include "IMaterialBakingAdapter.h"
#include "StaticMeshComponentLODInfo.h"
#include "SkeletalMeshAdapter.h"
#include "StaticMeshAdapter.h"

#include "MeshMergeDataTracker.h"

#include "Misc/FileHelper.h"
#include "MeshMergeHelpers.h"
#include "Settings/EditorExperimentalSettings.h"
#include "MaterialBakingStructures.h"
#include "Async/ParallelFor.h"
#include "ScopedTransaction.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/LODActor.h"
#include "HierarchicalLODVolume.h"
#include "Engine/Selection.h"
#include "MaterialBakingHelpers.h"
#include "IMeshMergeExtension.h"

#include "RawMesh.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "TriangleTypes.h"
#include "MaterialUtilities.h"

#include "Async/Future.h"
#include "Async/Async.h"
#include "TextureCompiler.h"

#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"

#include "ISMPartition/ISMComponentBatcher.h"
#include "ISMPartition/ISMComponentDescriptor.h"

#include "UObject/GCObjectScopeGuard.h"

#include "Algo/RemoveIf.h"

#define LOCTEXT_NAMESPACE "MeshMergeUtils"

DEFINE_LOG_CATEGORY(LogMeshMerging);

static TAutoConsoleVariable<int32> CVarMeshMergeUtilitiesUVGenerationMethod(
	TEXT("MeshMergeUtilities.UVGenerationMethod"),
	0,
	TEXT("UV generation method when creating merged or proxy meshes\n"
		 "0 - Engine default - (currently Patch Builder)\n"
		 "1 - Legacy\n"
		 "2 - UVAtlas\n"
		 "3 - XAtlas\n"
		 "4 - Patch Builder\n"));

static FStaticMeshOperations::EGenerateUVMethod GetUVGenerationMethodToUse()
{
	switch (CVarMeshMergeUtilitiesUVGenerationMethod.GetValueOnAnyThread())
	{
	case 1:  return FStaticMeshOperations::EGenerateUVMethod::Legacy;
	case 2:  return FStaticMeshOperations::EGenerateUVMethod::UVAtlas;
	case 3:  return FStaticMeshOperations::EGenerateUVMethod::XAtlas;
	case 4:  return FStaticMeshOperations::EGenerateUVMethod::PatchBuilder;
	default: return FStaticMeshOperations::EGenerateUVMethod::Default;
	}
}

FMeshMergeUtilities::FMeshMergeUtilities()
{
	Processor = new FProxyGenerationProcessor(this);
}

FMeshMergeUtilities::~FMeshMergeUtilities()
{
	FModuleManager::Get().OnModulesChanged().Remove(ModuleLoadedDelegateHandle);
}

void FMeshMergeUtilities::BakeMaterialsForComponent(TArray<TWeakObjectPtr<UObject>>& OptionObjects, IMaterialBakingAdapter* Adapter) const
{
	// Try and find material (merge) options from provided set of objects
	TWeakObjectPtr<UObject>* MaterialOptionsObject = OptionObjects.FindByPredicate([](TWeakObjectPtr<UObject> Object)
	{
		return Cast<UMaterialOptions>(Object.Get()) != nullptr;
	});

	TWeakObjectPtr<UObject>* MaterialMergeOptionsObject = OptionObjects.FindByPredicate([](TWeakObjectPtr<UObject> Object)
	{
		return Cast<UMaterialMergeOptions>(Object.Get()) != nullptr;
	});

	UMaterialOptions* MaterialOptions = MaterialOptionsObject ? Cast<UMaterialOptions>(MaterialOptionsObject->Get()) : nullptr;
	checkf(MaterialOptions, TEXT("No valid material options found"));


	UMaterialMergeOptions* MaterialMergeOptions  = MaterialMergeOptionsObject ? Cast<UMaterialMergeOptions>(MaterialMergeOptionsObject->Get()) : nullptr;

	// Mesh / LOD index	
	TMap<uint32, FMeshDescription> RawMeshLODs;

	// Unique set of sections in mesh
	TArray<FSectionInfo> UniqueSections;

	TArray<FSectionInfo> Sections;

	int32 NumLODs = Adapter->GetNumberOfLODs();

	// LOD index, <original section index, unique section index>
	TArray<TMap<int32, int32>> UniqueSectionIndexPerLOD;
	UniqueSectionIndexPerLOD.AddDefaulted(NumLODs);

	// Retrieve raw mesh data and unique sections
	for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
	{
		// Reset section for reuse
		Sections.SetNum(0, EAllowShrinking::No);

		// Extract raw mesh data 
		const bool bProcessedLOD = MaterialOptions->LODIndices.Contains(LODIndex);
		if (bProcessedLOD)
		{
			FMeshDescription& RawMesh = RawMeshLODs.Add(LODIndex);
			FStaticMeshAttributes(RawMesh).Register();
			Adapter->RetrieveRawMeshData(LODIndex, RawMesh, MaterialOptions->bUseMeshData);
		}

		// Extract sections for given LOD index from the mesh 
		Adapter->RetrieveMeshSections(LODIndex, Sections);

		for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
		{
			FSectionInfo& Section = Sections[SectionIndex];
			Section.bProcessed = bProcessedLOD;
			const int32 UniqueIndex = UniqueSections.AddUnique(Section);
			UniqueSectionIndexPerLOD[LODIndex].Emplace(SectionIndex, UniqueIndex);
		}
	}

	TArray<UMaterialInterface*> UniqueMaterials;
	TMultiMap<uint32, uint32> UniqueMaterialToUniqueSectionMap;
	// Populate list of unique materials and store section mappings
	for (int32 SectionIndex = 0; SectionIndex < UniqueSections.Num(); ++SectionIndex)
	{
		FSectionInfo& Section = UniqueSections[SectionIndex];
		const int32 UniqueIndex = UniqueMaterials.AddUnique(Section.Material);
		UniqueMaterialToUniqueSectionMap.Add(UniqueIndex, SectionIndex);
	}

	TArray<FMeshData> GlobalMeshSettings;
	TArray<FMaterialData> GlobalMaterialSettings;
	TArray<TMap<uint32, uint32>> OutputMaterialsMap;
	OutputMaterialsMap.AddDefaulted(NumLODs);

	for (int32 MaterialIndex = 0; MaterialIndex < UniqueMaterials.Num(); ++MaterialIndex)
	{
		UMaterialInterface* Material = UniqueMaterials[MaterialIndex];

		// Retrieve all sections using this material 
		TArray<uint32> SectionIndices;
		UniqueMaterialToUniqueSectionMap.MultiFind(MaterialIndex, SectionIndices);

		if (MaterialOptions->bUseMeshData)
		{
			for (const int32 LODIndex : MaterialOptions->LODIndices)
			{
				FMeshData MeshSettings;
				MeshSettings.MeshDescription = nullptr;

				// Add material indices used for rendering out material
				for (const auto& Pair : UniqueSectionIndexPerLOD[LODIndex])
				{
					if (SectionIndices.Contains(Pair.Value))
					{
						MeshSettings.MaterialIndices.Add(Pair.Key);
					}
				}

				if (MeshSettings.MaterialIndices.Num())
				{
					// Retrieve raw mesh
					MeshSettings.MeshDescription = RawMeshLODs.Find(LODIndex);
					
					//Should not be using mesh data if there is no mesh
					check(MeshSettings.MeshDescription);

					MeshSettings.TextureCoordinateBox = FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f));
					const bool bUseVertexColor = FStaticMeshOperations::HasVertexColor(*(MeshSettings.MeshDescription));
					if (MaterialOptions->bUseSpecificUVIndex)
					{
						MeshSettings.TextureCoordinateIndex = MaterialOptions->TextureCoordinateIndex;
					}
					// if you use vertex color, we can't rely on overlapping UV channel, so use light map UV to unwrap UVs
					else if (bUseVertexColor)
					{
						MeshSettings.TextureCoordinateIndex = Adapter->LightmapUVIndex();
					}
					else
					{
						MeshSettings.TextureCoordinateIndex = 0;
					}
					
					Adapter->ApplySettings(LODIndex, MeshSettings);
					
					// In case part of the UVs is not within the 0-1 range try to use the lightmap UVs
					const bool bNeedsUniqueUVs = FMeshMergeHelpers::CheckWrappingUVs(*(MeshSettings.MeshDescription), MeshSettings.TextureCoordinateIndex);
					const int32 LightMapUVIndex = Adapter->LightmapUVIndex();
					
					TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = FStaticMeshConstAttributes(*MeshSettings.MeshDescription).GetVertexInstanceUVs();
					if (bNeedsUniqueUVs && MeshSettings.TextureCoordinateIndex != LightMapUVIndex && VertexInstanceUVs.GetNumElements() > 0 && VertexInstanceUVs.GetNumChannels() > LightMapUVIndex)
					{
						MeshSettings.TextureCoordinateIndex = LightMapUVIndex;
					}

					FMaterialData MaterialSettings;
					MaterialSettings.Material = Material;					

					// Add all user defined properties for baking out
					for (const FPropertyEntry& Entry : MaterialOptions->Properties)
					{
						if (!Entry.bUseConstantValue && Entry.Property != MP_MAX)
						{
							int32 NumTextureCoordinates;
							bool bUsesVertexData;
							Material->AnalyzeMaterialProperty(Entry.Property, NumTextureCoordinates, bUsesVertexData);

							MaterialSettings.PropertySizes.Add(Entry.Property, Entry.bUseCustomSize ? Entry.CustomSize : MaterialOptions->TextureSize);
						}
					}

					// For each original material index add an entry to the corresponding LOD and bake output index 
					for (int32 Index : MeshSettings.MaterialIndices)
					{
						OutputMaterialsMap[LODIndex].Emplace(Index, GlobalMeshSettings.Num());
					}

					GlobalMeshSettings.Add(MeshSettings);
					GlobalMaterialSettings.Add(MaterialSettings);
				}
			}
		}
		else
		{
			// If we are not using the mesh data we aren't doing anything special, just bake out uv range
			FMeshData MeshSettings;
			for (int32 LODIndex : MaterialOptions->LODIndices)
			{
				for (const auto& Pair : UniqueSectionIndexPerLOD[LODIndex])
				{
					if (SectionIndices.Contains(Pair.Value))
					{
						MeshSettings.MaterialIndices.Add(Pair.Key);
					}
				}
			}

			if (MeshSettings.MaterialIndices.Num())
			{
				MeshSettings.MeshDescription = nullptr;
				MeshSettings.TextureCoordinateBox = FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f));
				MeshSettings.TextureCoordinateIndex = 0;

				FMaterialData MaterialSettings;
				MaterialSettings.Material = Material;

				// Add all user defined properties for baking out
				for (const FPropertyEntry& Entry : MaterialOptions->Properties)
				{
					// With substrate, we cannot use connection on the root node to infer Active or Constant properties. So we activate all material input for now. 
					// SUBSTRATE_TODO: gather information about the material used input, maybe using Material->GetSubstrateCompilationConfig().
					const float bIsSubstrateEnabled = Substrate::IsSubstrateEnabled();

					if ((bIsSubstrateEnabled || (!Entry.bUseConstantValue && Material->IsPropertyActive(Entry.Property))) && Entry.Property != MP_MAX)
					{
						MaterialSettings.PropertySizes.Add(Entry.Property, Entry.bUseCustomSize ? Entry.CustomSize : MaterialOptions->TextureSize);
					}
				}

				for (int32 LODIndex : MaterialOptions->LODIndices)
				{
					for (const auto& Pair : UniqueSectionIndexPerLOD[LODIndex])
					{
						if (SectionIndices.Contains(Pair.Value))
						{
							/// For each original material index add an entry to the corresponding LOD and bake output index 
							OutputMaterialsMap[LODIndex].Emplace(Pair.Key, GlobalMeshSettings.Num());
						}
					}
				}

				GlobalMeshSettings.Add(MeshSettings);
				GlobalMaterialSettings.Add(MaterialSettings);
			}
		}
	}

	TArray<FMeshData*> MeshSettingPtrs;
	for (int32 SettingsIndex = 0; SettingsIndex < GlobalMeshSettings.Num(); ++SettingsIndex)
	{
		MeshSettingPtrs.Add(&GlobalMeshSettings[SettingsIndex]);
	}

	TArray<FMaterialData*> MaterialSettingPtrs;
	for (int32 SettingsIndex = 0; SettingsIndex < GlobalMaterialSettings.Num(); ++SettingsIndex)
	{
		MaterialSettingPtrs.Add(&GlobalMaterialSettings[SettingsIndex]);
	}

	TArray<FBakeOutput> BakeOutputs;
	IMaterialBakingModule& Module = FModuleManager::Get().LoadModuleChecked<IMaterialBakingModule>("MaterialBaking");
	Module.BakeMaterials(MaterialSettingPtrs, MeshSettingPtrs, BakeOutputs);

	// Append constant properties which did not require baking out
	TArray<FColor> ConstantData;
	FIntPoint ConstantSize(1, 1);
	for (const FPropertyEntry& Entry : MaterialOptions->Properties)
	{
		if (Entry.bUseConstantValue && Entry.Property != MP_MAX)
		{
			ConstantData.SetNum(1, EAllowShrinking::No);
			ConstantData[0] = FColor(Entry.ConstantValue * 255.0f, Entry.ConstantValue * 255.0f, Entry.ConstantValue * 255.0f);
			for (FBakeOutput& Ouput : BakeOutputs)
			{
				Ouput.PropertyData.Add(Entry.Property, ConstantData);
				Ouput.PropertySizes.Add(Entry.Property, ConstantSize);
			}
		}
	}

	TArray<UMaterialInterface*> NewMaterials;

	FString PackageName = Adapter->GetBaseName();

	const FGuid NameGuid = FGuid::NewGuid();
	for (int32 OutputIndex = 0; OutputIndex < BakeOutputs.Num(); ++OutputIndex)
	{
		// Create merged material asset
		FString MaterialAssetName = TEXT("M_") + FPackageName::GetShortName(PackageName) + TEXT("_") + MaterialSettingPtrs[OutputIndex]->Material->GetName() + TEXT("_") + NameGuid.ToString();
		FString MaterialPackageName = FPackageName::GetLongPackagePath(PackageName) + TEXT("/") + MaterialAssetName;

		FBakeOutput& Output = BakeOutputs[OutputIndex];
		// Optimize output 
		for (auto DataPair : Output.PropertyData)
		{
			FMaterialUtilities::OptimizeSampleArray(DataPair.Value, Output.PropertySizes[DataPair.Key]);
		}

		UMaterialInterface* Material = nullptr;

		if (Adapter->GetOuter())
		{
			Material = FMaterialUtilities::CreateProxyMaterialAndTextures(Adapter->GetOuter(), MaterialAssetName, Output, *MeshSettingPtrs[OutputIndex], *MaterialSettingPtrs[OutputIndex], MaterialOptions);
		}
		else
		{
			Material = FMaterialUtilities::CreateProxyMaterialAndTextures(MaterialPackageName, MaterialAssetName, Output, *MeshSettingPtrs[OutputIndex], *MaterialSettingPtrs[OutputIndex], MaterialOptions);
		}

		
		NewMaterials.Add(Material);
	}

	// Retrieve material indices which were not baked out and should still be part of the final asset
	TArray<int32> NonReplaceMaterialIndices;
	for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
	{
		const bool bProcessedLOD = MaterialOptions->LODIndices.Contains(LODIndex);
		if (!bProcessedLOD)
		{
			for (const auto& Pair : UniqueSectionIndexPerLOD[LODIndex])
			{
				NonReplaceMaterialIndices.AddUnique(Adapter->GetMaterialIndex(LODIndex, Pair.Key));
			}
		}
	}

	// Remap all baked out materials to their new material indices
	TMap<uint32, uint32> NewMaterialRemap;
	for (int32 LODIndex : MaterialOptions->LODIndices)
	{
		// Key == original section index, Value == unique material index
		for (const auto& Pair : OutputMaterialsMap[LODIndex])
		{
			int32 SetIndex = Adapter->GetMaterialIndex(LODIndex, Pair.Key);
			if (!NonReplaceMaterialIndices.Contains(SetIndex))
			{
				Adapter->SetMaterial(SetIndex, NewMaterials[Pair.Value]);
			}
			else
			{
				// Check if this material was  processed and a new entry already exists
				if (uint32* ExistingIndex = NewMaterialRemap.Find(Pair.Value))
				{
					Adapter->RemapMaterialIndex(LODIndex, Pair.Key, *ExistingIndex);
				}
				else
				{
					// Add new material
					int32 NewMaterialIndex = INDEX_NONE;
					if (Adapter->GetMaterialSlotName(Pair.Key).IsNone() || Adapter->GetImportedMaterialSlotName(Pair.Key).IsNone())
					{
						NewMaterialIndex = Adapter->AddMaterial(NewMaterials[Pair.Value]);
					}
					else
					{
						NewMaterialIndex = Adapter->AddMaterial(NewMaterials[Pair.Value], Adapter->GetMaterialSlotName(Pair.Key), Adapter->GetImportedMaterialSlotName(Pair.Key));
					}

					NewMaterialRemap.Add(Pair.Value, NewMaterialIndex);
					Adapter->RemapMaterialIndex(LODIndex, Pair.Key, NewMaterialIndex);
				}
			}
		}
	}

	Adapter->UpdateUVChannelData();
	GlobalMeshSettings.Empty();
}

void FMeshMergeUtilities::BakeMaterialsForComponent(USkeletalMeshComponent* SkeletalMeshComponent) const
{
	if (!SkeletalMeshComponent || !SkeletalMeshComponent->GetSkeletalMeshAsset())
	{
		return;
	}

	// Retrieve settings object
	UMaterialOptions* MaterialOptions = DuplicateObject(GetMutableDefault<UMaterialOptions>(), GetTransientPackage());
	UAssetBakeOptions* AssetOptions = GetMutableDefault<UAssetBakeOptions>();
	UMaterialMergeOptions* MergeOptions = GetMutableDefault<UMaterialMergeOptions>();
	TArray<TWeakObjectPtr<UObject>> Objects{ MergeOptions, AssetOptions, MaterialOptions };

	const int32 NumLODs = SkeletalMeshComponent->GetSkeletalMeshAsset()->GetLODNum();
	IMaterialBakingModule& Module = FModuleManager::Get().LoadModuleChecked<IMaterialBakingModule>("MaterialBaking");
	if (!Module.SetupMaterialBakeSettings(Objects, NumLODs))
	{
		return;
	}

	// Bake out materials for skeletal mesh
	SkeletalMeshComponent->GetSkeletalMeshAsset()->Modify();
	FSkeletalMeshComponentAdapter Adapter(SkeletalMeshComponent);
	BakeMaterialsForComponent(Objects, &Adapter);
	SkeletalMeshComponent->MarkRenderStateDirty();
}

void FMeshMergeUtilities::BakeMaterialsForComponent(UStaticMeshComponent* StaticMeshComponent) const
{
	if (!StaticMeshComponent || !StaticMeshComponent->GetStaticMesh())
	{
		return;
	}

	// Retrieve settings object
	UMaterialOptions* MaterialOptions = DuplicateObject(GetMutableDefault<UMaterialOptions>(), GetTransientPackage());
	UAssetBakeOptions* AssetOptions = GetMutableDefault<UAssetBakeOptions>();
	UMaterialMergeOptions* MergeOptions = GetMutableDefault<UMaterialMergeOptions>();
	TArray<TWeakObjectPtr<UObject>> Objects{ MergeOptions, AssetOptions, MaterialOptions };

	const int32 NumLODs = StaticMeshComponent->GetStaticMesh()->GetNumLODs();
	IMaterialBakingModule& Module = FModuleManager::Get().LoadModuleChecked<IMaterialBakingModule>("MaterialBaking");
	if (!Module.SetupMaterialBakeSettings(Objects, NumLODs))
	{
		return;
	}

	// Bake out materials for static mesh component
	FStaticMeshComponentAdapter Adapter(StaticMeshComponent);
	BakeMaterialsForComponent(Objects, &Adapter);
	StaticMeshComponent->MarkRenderStateDirty();
}

void FMeshMergeUtilities::BakeMaterialsForMesh(UStaticMesh* StaticMesh) const
{
	if (!StaticMesh)
	{
		return;
	}

	// Retrieve settings object
	UMaterialOptions* MaterialOptions = DuplicateObject(GetMutableDefault<UMaterialOptions>(), GetTransientPackage());
	UAssetBakeOptions* AssetOptions = GetMutableDefault<UAssetBakeOptions>();
	UMaterialMergeOptions* MergeOptions = GetMutableDefault<UMaterialMergeOptions>();
	TArray<TWeakObjectPtr<UObject>> Objects{ MergeOptions, AssetOptions, MaterialOptions };

	const int32 NumLODs = StaticMesh->GetNumLODs();
	IMaterialBakingModule& Module = FModuleManager::Get().LoadModuleChecked<IMaterialBakingModule>("MaterialBaking");
	if (!Module.SetupMaterialBakeSettings(Objects, NumLODs))
	{
		return;
	}

	// Bake out materials for static mesh asset
	StaticMesh->Modify();
	FStaticMeshAdapter Adapter(StaticMesh);
	BakeMaterialsForComponent(Objects, &Adapter);
}


struct FMaterialDataUsage
{
	bool bUsesVertexData = false;
	bool bUsesWorldPosition = false;
};

static FMaterialDataUsage DetermineMaterialDataUsage(UMaterialInterface* Material, const UMaterialOptions* MaterialOptions)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DetermineMaterialDataUsage);

	FMaterialDataUsage Usage;

	for (const FPropertyEntry& Entry : MaterialOptions->Properties)
	{
		// Don't have to check a property if the result is going to be constant anyway
		if (!Entry.bUseConstantValue && Entry.Property != MP_MAX)
		{
			FMaterialAnalysisResult Result;
			Material->AnalyzeMaterialPropertyEx(Entry.Property, Result);

			int32 NumTextureCoordinates = Result.TextureCoordinates.FindLast(true) + 1;

			Usage.bUsesVertexData |= Result.bRequiresVertexData || NumTextureCoordinates > 1;
			Usage.bUsesWorldPosition |= Result.bRequiresWorldPosition;

			if (Usage.bUsesVertexData && Usage.bUsesWorldPosition)
			{
				break;
			}
		}
	}

	return Usage;
}

void FMeshMergeUtilities::ConvertOutputToFlatMaterials(const TArray<FBakeOutput>& BakeOutputs, const TArray<FMaterialData>& MaterialData, TArray<FFlattenMaterial> &FlattenedMaterials) const
{
	for (int32 OutputIndex = 0; OutputIndex < BakeOutputs.Num(); ++OutputIndex)
	{
		const FBakeOutput& Output = BakeOutputs[OutputIndex];
		const FMaterialData& MaterialInfo = MaterialData[OutputIndex];

		FFlattenMaterial Material;		

		for (TPair<EMaterialProperty, FIntPoint> SizePair : Output.PropertySizes)
		{
			EFlattenMaterialProperties OldProperty = ToFlattenProperty(SizePair.Key);
			if (ensure(OldProperty != EFlattenMaterialProperties::NumFlattenMaterialProperties))
			{
				Material.SetPropertySize(OldProperty, SizePair.Value);
				Material.GetPropertySamples(OldProperty).Append(Output.PropertyData[SizePair.Key]);
			}
		}

		Material.bDitheredLODTransition = MaterialInfo.Material->IsDitheredLODTransition();
		Material.BlendMode = BLEND_Opaque;
		Material.bTwoSided = MaterialInfo.Material->IsTwoSided();
		Material.EmissiveScale = Output.EmissiveScale;

		FlattenedMaterials.Add(Material);
	}
}

void FMeshMergeUtilities::TransferOutputToFlatMaterials(const TArray<FMaterialData>& InMaterialData, TArray<FBakeOutput>& InOutBakeOutputs, TArray<FFlattenMaterial> &OutFlattenedMaterials) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMergeUtilities::TransferOutputToFlatMaterials)

	OutFlattenedMaterials.SetNum(InOutBakeOutputs.Num());

	for (int32 OutputIndex = 0; OutputIndex < InOutBakeOutputs.Num(); ++OutputIndex)
	{
		FBakeOutput& Output = InOutBakeOutputs[OutputIndex];
		const FMaterialData& MaterialInfo = InMaterialData[OutputIndex];

		FFlattenMaterial& Material = OutFlattenedMaterials[OutputIndex];

		for (TPair<EMaterialProperty, FIntPoint> SizePair : Output.PropertySizes)
		{
			EFlattenMaterialProperties OldProperty = ToFlattenProperty(SizePair.Key);
			if (ensure(OldProperty != EFlattenMaterialProperties::NumFlattenMaterialProperties))
			{
				Material.SetPropertySize(OldProperty, SizePair.Value);
				Material.GetPropertySamples(OldProperty) = MoveTemp(Output.PropertyData[SizePair.Key]);
			}
		}

		Material.bDitheredLODTransition = MaterialInfo.Material->IsDitheredLODTransition();
		Material.BlendMode = BLEND_Opaque;
		Material.bTwoSided = MaterialInfo.Material->IsTwoSided();
		Material.EmissiveScale = Output.EmissiveScale;
	}
}

EFlattenMaterialProperties FMeshMergeUtilities::ToFlattenProperty(EMaterialProperty MaterialProperty) const
{
	switch (MaterialProperty)
	{
	case EMaterialProperty::MP_BaseColor:			return EFlattenMaterialProperties::Diffuse;
	case EMaterialProperty::MP_Metallic:			return EFlattenMaterialProperties::Metallic;
	case EMaterialProperty::MP_Specular:			return EFlattenMaterialProperties::Specular;
	case EMaterialProperty::MP_Roughness:			return EFlattenMaterialProperties::Roughness;
	case EMaterialProperty::MP_Anisotropy:			return EFlattenMaterialProperties::Anisotropy;
	case EMaterialProperty::MP_Normal:				return EFlattenMaterialProperties::Normal;
	case EMaterialProperty::MP_Tangent:				return EFlattenMaterialProperties::Tangent;
	case EMaterialProperty::MP_Opacity:				return EFlattenMaterialProperties::Opacity;
	case EMaterialProperty::MP_EmissiveColor:		return EFlattenMaterialProperties::Emissive;
	case EMaterialProperty::MP_OpacityMask:			return EFlattenMaterialProperties::OpacityMask;
	case EMaterialProperty::MP_AmbientOcclusion:	return EFlattenMaterialProperties::AmbientOcclusion;
	default:										return EFlattenMaterialProperties::NumFlattenMaterialProperties;
	}
}

UMaterialOptions* FMeshMergeUtilities::PopulateMaterialOptions(const FMaterialProxySettings& MaterialSettings) const
{
	UMaterialOptions* MaterialOptions = DuplicateObject(GetMutableDefault<UMaterialOptions>(), GetTransientPackage());
	MaterialOptions->Properties.Empty();	
	MaterialOptions->TextureSize = MaterialSettings.TextureSize;
	
	FPropertyEntry Property;
	PopulatePropertyEntry(MaterialSettings, MP_BaseColor, Property);
	MaterialOptions->Properties.Add(Property);

	PopulatePropertyEntry(MaterialSettings, MP_Specular, Property);
	if (MaterialSettings.bSpecularMap)
		MaterialOptions->Properties.Add(Property);

	PopulatePropertyEntry(MaterialSettings, MP_Roughness, Property);
	if (MaterialSettings.bRoughnessMap)
		MaterialOptions->Properties.Add(Property);

	PopulatePropertyEntry(MaterialSettings, MP_Anisotropy, Property);
	if (MaterialSettings.bAnisotropyMap)
	{
		MaterialOptions->Properties.Add(Property);
	}

	PopulatePropertyEntry(MaterialSettings, MP_Metallic, Property);
	if (MaterialSettings.bMetallicMap)
		MaterialOptions->Properties.Add(Property);

	PopulatePropertyEntry(MaterialSettings, MP_Normal, Property);
	if (MaterialSettings.bNormalMap)
		MaterialOptions->Properties.Add(Property);

	PopulatePropertyEntry(MaterialSettings, MP_Tangent, Property);
	if (MaterialSettings.bTangentMap)
	{
		MaterialOptions->Properties.Add(Property);
	}

	PopulatePropertyEntry(MaterialSettings, MP_Opacity, Property);
	if (MaterialSettings.bOpacityMap)
		MaterialOptions->Properties.Add(Property);

	PopulatePropertyEntry(MaterialSettings, MP_OpacityMask, Property);
	if (MaterialSettings.bOpacityMaskMap)
		MaterialOptions->Properties.Add(Property);

	PopulatePropertyEntry(MaterialSettings, MP_EmissiveColor, Property);
	if (MaterialSettings.bEmissiveMap)
		MaterialOptions->Properties.Add(Property);

	PopulatePropertyEntry(MaterialSettings, MP_AmbientOcclusion, Property);
	if (MaterialSettings.bAmbientOcclusionMap)
		MaterialOptions->Properties.Add(Property);

	return MaterialOptions;
}

void FMeshMergeUtilities::PopulatePropertyEntry(const FMaterialProxySettings& MaterialSettings, EMaterialProperty MaterialProperty, FPropertyEntry& InOutPropertyEntry) const
{
	InOutPropertyEntry.Property = MaterialProperty;
	switch (MaterialSettings.TextureSizingType)
	{	
		/** Set property output size to unique per-property user set sizes */
		case TextureSizingType_UseManualOverrideTextureSize:
		{
			InOutPropertyEntry.bUseCustomSize = true;
			InOutPropertyEntry.CustomSize = [MaterialSettings, MaterialProperty]() -> FIntPoint
			{
				switch (MaterialProperty)
				{
					case MP_BaseColor: return MaterialSettings.DiffuseTextureSize;
					case MP_Specular: return MaterialSettings.SpecularTextureSize;
					case MP_Roughness: return MaterialSettings.RoughnessTextureSize;
					case MP_Anisotropy: return MaterialSettings.AnisotropyTextureSize;
					case MP_Metallic: return MaterialSettings.MetallicTextureSize;
					case MP_Normal: return MaterialSettings.NormalTextureSize;
					case MP_Tangent: return MaterialSettings.TangentTextureSize;
					case MP_Opacity: return MaterialSettings.OpacityTextureSize;
					case MP_OpacityMask: return MaterialSettings.OpacityMaskTextureSize;
					case MP_EmissiveColor: return MaterialSettings.EmissiveTextureSize;
					case MP_AmbientOcclusion: return MaterialSettings.AmbientOcclusionTextureSize;
					default:
					{
						checkf(false, TEXT("Invalid Material Property"));
						return FIntPoint();
					}	
				}
			}();

			break;
		}
		/** Set property output size to biased values off the TextureSize value (Normal at fullres, Diffuse at halfres, and anything else at quarter res */
		case TextureSizingType_UseAutomaticBiasedSizes:
		{
			const FIntPoint FullRes = MaterialSettings.TextureSize;
			const FIntPoint HalfRes = FIntPoint(FMath::Max(8, FullRes.X >> 1), FMath::Max(8, FullRes.Y >> 1));
			const FIntPoint QuarterRes = FIntPoint(FMath::Max(4, FullRes.X >> 2), FMath::Max(4, FullRes.Y >> 2));

			InOutPropertyEntry.bUseCustomSize = true;
			InOutPropertyEntry.CustomSize = [FullRes, HalfRes, QuarterRes, MaterialSettings, MaterialProperty]() -> FIntPoint
			{
				switch (MaterialProperty)
				{
				case MP_Normal: return FullRes;
				case MP_Tangent: return HalfRes;
				case MP_BaseColor: return HalfRes;
				case MP_Specular: return QuarterRes;
				case MP_Roughness: return QuarterRes;
				case MP_Anisotropy: return QuarterRes;
				case MP_Metallic: return QuarterRes;				
				case MP_Opacity: return QuarterRes;
				case MP_OpacityMask: return QuarterRes;
				case MP_EmissiveColor: return QuarterRes;
				case MP_AmbientOcclusion: return QuarterRes;
				default:
				{
					checkf(false, TEXT("Invalid Material Property"));
					return FIntPoint();
				}
				}
			}();

			break;
		}
 		/** Set all sizes to TextureSize */
		case TextureSizingType_UseSingleTextureSize:
		case TextureSizingType_UseSimplygonAutomaticSizing:
		{
			InOutPropertyEntry.bUseCustomSize = false;
			InOutPropertyEntry.CustomSize = MaterialSettings.TextureSize;
			break;
		}

		default:
			UE_LOGF(LogMeshMerging, Error, "Unsupported TextureSizingType value. You should resolve the material texture size first with ResolveTextureSize()");

	}
	/** Check whether or not a constant value should be used for this property */
	InOutPropertyEntry.bUseConstantValue = [MaterialSettings, MaterialProperty]() -> bool
	{
		switch (MaterialProperty)
		{
			case MP_BaseColor: return false;
			case MP_Normal: return !MaterialSettings.bNormalMap;
			case MP_Tangent: return !MaterialSettings.bTangentMap;
			case MP_Specular: return !MaterialSettings.bSpecularMap;
			case MP_Roughness: return !MaterialSettings.bRoughnessMap;
			case MP_Anisotropy: return !MaterialSettings.bAnisotropyMap;
			case MP_Metallic: return !MaterialSettings.bMetallicMap;
			case MP_Opacity: return !MaterialSettings.bOpacityMap;
			case MP_OpacityMask: return !MaterialSettings.bOpacityMaskMap;
			case MP_EmissiveColor: return !MaterialSettings.bEmissiveMap;
			case MP_AmbientOcclusion: return !MaterialSettings.bAmbientOcclusionMap;
			default:
			{
				checkf(false, TEXT("Invalid Material Property"));
				return false;
			}
		}
	}();
	/** Set the value if a constant value should be used for this property */
	InOutPropertyEntry.ConstantValue = [MaterialSettings, MaterialProperty]() -> float
	{
		switch (MaterialProperty)
		{
			case MP_BaseColor: return 1.0f;
			case MP_Normal: return 1.0f;
			case MP_Tangent: return 1.0f;
			case MP_Specular: return MaterialSettings.SpecularConstant;
			case MP_Roughness: return MaterialSettings.RoughnessConstant;
			case MP_Anisotropy: return MaterialSettings.AnisotropyConstant;
			case MP_Metallic: return MaterialSettings.MetallicConstant;
			case MP_Opacity: return MaterialSettings.OpacityConstant;
			case MP_OpacityMask: return MaterialSettings.OpacityMaskConstant;
			case MP_EmissiveColor: return 0.0f;
			case MP_AmbientOcclusion: return MaterialSettings.AmbientOcclusionConstant;
			default:
			{
				checkf(false, TEXT("Invalid Material Property"));
				return 1.0f;
			}
		}
	}();
}

void FMeshMergeUtilities::CopyTextureRect(const FColor* Src, const FIntPoint& SrcSize, FColor* Dst, const FIntPoint& DstSize, const FIntPoint& DstPos, bool bCopyOnlyMaskedPixels) const
{
	const int32 RowLength = SrcSize.X * sizeof(FColor);
	FColor* RowDst = Dst + DstSize.X*DstPos.Y;
	const FColor* RowSrc = Src;
	if(bCopyOnlyMaskedPixels)
	{
		for (int32 RowIdx = 0; RowIdx < SrcSize.Y; ++RowIdx)
		{
			for (int32 ColIdx = 0; ColIdx < SrcSize.X; ++ColIdx)
			{
				if(RowSrc[ColIdx] != FColor::Magenta)
				{
					RowDst[DstPos.X + ColIdx] = RowSrc[ColIdx];
				}
			}

			RowDst += DstSize.X;
			RowSrc += SrcSize.X;
		}
	}
	else
	{
		for (int32 RowIdx = 0; RowIdx < SrcSize.Y; ++RowIdx)
		{
			FMemory::Memcpy(RowDst + DstPos.X, RowSrc, RowLength);
			RowDst += DstSize.X;
			RowSrc += SrcSize.X;
		}
	}
}

void FMeshMergeUtilities::SetTextureRect(const FColor& ColorValue, const FIntPoint& SrcSize, FColor* Dst, const FIntPoint& DstSize, const FIntPoint& DstPos) const
{
	FColor* RowDst = Dst + DstSize.X*DstPos.Y;

	for (int32 RowIdx = 0; RowIdx < SrcSize.Y; ++RowIdx)
	{
		for (int32 ColIdx = 0; ColIdx < SrcSize.X; ++ColIdx)
		{
			RowDst[DstPos.X + ColIdx] = ColorValue;
		}

		RowDst += DstSize.X;
	}
}

FIntPoint FMeshMergeUtilities::ConditionalImageResize(const FIntPoint& SrcSize, const FIntPoint& DesiredSize, TArray<FColor>& InOutImage, bool bLinearSpace) const
{
	const int32 NumDesiredSamples = DesiredSize.X*DesiredSize.Y;
	if (InOutImage.Num() && InOutImage.Num() != NumDesiredSamples)
	{
		check(InOutImage.Num() == SrcSize.X*SrcSize.Y);
		TArray<FColor> OutImage;
		if (NumDesiredSamples > 0)
		{
			const bool bOpaqueAlpha = true; // ??
			FImageUtils::ImageResize(SrcSize.X, SrcSize.Y, InOutImage, DesiredSize.X, DesiredSize.Y, OutImage, bLinearSpace, bOpaqueAlpha);
		}
		Exchange(InOutImage, OutImage);
		return DesiredSize;
	}

	return SrcSize;
}

void FMeshMergeUtilities::MergeFlattenedMaterials(TArray<struct FFlattenMaterial>& InMaterialList, int32 InGutter, FFlattenMaterial& OutMergedMaterial, TArray<FUVOffsetScalePair>& OutUVTransforms) const
{
	// Fill output UV transforms with invalid values
	OutUVTransforms.SetNumZeroed(InMaterialList.Num());

	const int32 AtlasGridSize = FMath::CeilToInt(FMath::Sqrt(static_cast<float>(InMaterialList.Num())));
	OutMergedMaterial.EmissiveScale = FlattenEmissivescale(InMaterialList);

	for (int32 PropertyIndex = 0; PropertyIndex < (int32)EFlattenMaterialProperties::NumFlattenMaterialProperties; ++PropertyIndex)
	{
		const EFlattenMaterialProperties Property = (EFlattenMaterialProperties)PropertyIndex;
		if (OutMergedMaterial.ShouldGenerateDataForProperty(Property))
		{
			const FIntPoint AtlasTextureSize = OutMergedMaterial.GetPropertySize(Property);
			const FIntPoint ExportTextureSize = AtlasTextureSize / AtlasGridSize;
			const int32 AtlasNumSamples = AtlasTextureSize.X*AtlasTextureSize.Y;
			check(OutMergedMaterial.GetPropertySize(Property) == AtlasTextureSize);
			TArray<FColor>& Samples = OutMergedMaterial.GetPropertySamples(Property);
			Samples.SetNumUninitialized(AtlasNumSamples);

			// Fill with magenta (as we will be box blurring this later)
			for(FColor& SampleColor : Samples)
			{
				SampleColor = FColor(255, 0, 255);
			}
		}
	}

	int32 AtlasRowIdx = 0;
	int32 AtlasColIdx = 0;
	FIntPoint Gutter(InGutter, InGutter);
	FIntPoint DoubleGutter(InGutter * 2, InGutter * 2);
	FIntPoint GlobalAtlasTargetPos = Gutter;

	bool bSamplesWritten[(uint32)EFlattenMaterialProperties::NumFlattenMaterialProperties];
	FMemory::Memset(bSamplesWritten, 0);

	// Used to calculate UV transforms
	const FIntPoint GlobalAtlasTextureSize = OutMergedMaterial.GetPropertySize(EFlattenMaterialProperties::Diffuse);
	const FIntPoint GlobalExportTextureSize = (GlobalAtlasTextureSize / AtlasGridSize) - DoubleGutter;
	const FIntPoint GlobalExportEntrySize = (GlobalAtlasTextureSize / AtlasGridSize);

	// Flatten all materials and merge them into one material using texture atlases
	for (int32 MatIdx = 0; MatIdx < InMaterialList.Num(); ++MatIdx)
	{
		FFlattenMaterial& FlatMaterial = InMaterialList[MatIdx];
		OutMergedMaterial.bTwoSided |= FlatMaterial.bTwoSided;
		OutMergedMaterial.bDitheredLODTransition = FlatMaterial.bDitheredLODTransition;

		for (int32 PropertyIndex = 0; PropertyIndex < (int32)EFlattenMaterialProperties::NumFlattenMaterialProperties; ++PropertyIndex)
		{
			const EFlattenMaterialProperties Property = (EFlattenMaterialProperties)PropertyIndex;
			const FIntPoint PropertyTextureSize = OutMergedMaterial.GetPropertySize(Property);
			const int32 NumPropertySamples = PropertyTextureSize.X*PropertyTextureSize.Y;

			const FIntPoint PropertyAtlasTextureSize = (PropertyTextureSize / AtlasGridSize) - DoubleGutter;
			const FIntPoint PropertyAtlasEntrySize = (PropertyTextureSize / AtlasGridSize);
			const FIntPoint AtlasTargetPos((AtlasColIdx * PropertyAtlasEntrySize.X) + InGutter, (AtlasRowIdx * PropertyAtlasEntrySize.Y) + InGutter);
			
			if (OutMergedMaterial.ShouldGenerateDataForProperty(Property) && FlatMaterial.DoesPropertyContainData(Property))
			{
				TArray<FColor>& SourceSamples = FlatMaterial.GetPropertySamples(Property);
				TArray<FColor>& TargetSamples = OutMergedMaterial.GetPropertySamples(Property);
				if (FlatMaterial.IsPropertyConstant(Property))
				{
					SetTextureRect(SourceSamples[0], PropertyAtlasTextureSize, TargetSamples.GetData(), PropertyTextureSize, AtlasTargetPos);
				}
				else
				{
					FIntPoint PropertySize = FlatMaterial.GetPropertySize(Property);
					PropertySize = ConditionalImageResize(PropertySize, PropertyAtlasTextureSize, SourceSamples, false);
					CopyTextureRect(SourceSamples.GetData(), PropertyAtlasTextureSize, TargetSamples.GetData(), PropertyTextureSize, AtlasTargetPos);
					FlatMaterial.SetPropertySize(Property, PropertySize);
				}

				bSamplesWritten[PropertyIndex] |= true;
			}
		}

		check(OutUVTransforms.IsValidIndex(MatIdx));

		// Offset
		OutUVTransforms[MatIdx].Key = FVector2D(
			(float)GlobalAtlasTargetPos.X / GlobalAtlasTextureSize.X,
			(float)GlobalAtlasTargetPos.Y / GlobalAtlasTextureSize.Y);

		// Scale
		OutUVTransforms[MatIdx].Value = FVector2D(
			(float)GlobalExportTextureSize.X / GlobalAtlasTextureSize.X,
			(float)GlobalExportTextureSize.Y / GlobalAtlasTextureSize.Y);

		AtlasColIdx++;
		if (AtlasColIdx >= AtlasGridSize)
		{
			AtlasColIdx = 0;
			AtlasRowIdx++;
		}

		GlobalAtlasTargetPos = FIntPoint((AtlasColIdx * GlobalExportEntrySize.X) + InGutter, (AtlasRowIdx * GlobalExportEntrySize.Y) + InGutter);
	}

	// Check if some properties weren't populated with data (which means we can empty them out)
	for (int32 PropertyIndex = 0; PropertyIndex < (int32)EFlattenMaterialProperties::NumFlattenMaterialProperties; ++PropertyIndex)
	{
		EFlattenMaterialProperties Property = (EFlattenMaterialProperties)PropertyIndex;
		if (!bSamplesWritten[PropertyIndex])
		{	
			OutMergedMaterial.GetPropertySamples(Property).Empty();
			OutMergedMaterial.SetPropertySize(Property, FIntPoint(0, 0));
		}
		else
		{
			// Smear borders
			const FIntPoint PropertySize = OutMergedMaterial.GetPropertySize(Property);
			FMaterialBakingHelpers::PerformUVBorderSmear(OutMergedMaterial.GetPropertySamples(Property), PropertySize.X, PropertySize.Y);
		}
	}

}

void FMeshMergeUtilities::FlattenBinnedMaterials(TArray<struct FFlattenMaterial>& InMaterialList, const TArray<FBox2D>& InMaterialBoxes, int32 InGutter, bool bCopyOnlyMaskedPixels, FFlattenMaterial& OutMergedMaterial, TArray<FUVOffsetScalePair>& OutUVTransforms) const
{
	// Fill output UV transforms with invalid values
	OutUVTransforms.SetNumZeroed(InMaterialList.Num());

	// Flatten emissive scale across all incoming materials
	OutMergedMaterial.EmissiveScale = FlattenEmissivescale(InMaterialList);

	// Merge all material properties
	for (int32 Index = 0; Index < (int32)EFlattenMaterialProperties::NumFlattenMaterialProperties; ++Index)
	{
		const EFlattenMaterialProperties Property = (EFlattenMaterialProperties)Index;
		const FIntPoint& OutTextureSize = OutMergedMaterial.GetPropertySize(Property);
		if (OutTextureSize != FIntPoint::ZeroValue)
		{
			TArray<FColor>& OutSamples = OutMergedMaterial.GetPropertySamples(Property);
			OutSamples.Reserve(OutTextureSize.X * OutTextureSize.Y);
			OutSamples.SetNumUninitialized(OutTextureSize.X * OutTextureSize.Y);

			// Fill with magenta (as we will be box blurring this later)
			for(FColor& SampleColor : OutSamples)
			{
				SampleColor = FColor(255, 0, 255);
			}

			FVector2D Gutter2D((float)InGutter, (float)InGutter);
			bool bMaterialsWritten = false;
			for (int32 MaterialIndex = 0; MaterialIndex < InMaterialList.Num(); ++MaterialIndex)
			{
				// Determine output size and offset
				FFlattenMaterial& FlatMaterial = InMaterialList[MaterialIndex];
				OutMergedMaterial.bDitheredLODTransition |= FlatMaterial.bDitheredLODTransition;
				OutMergedMaterial.bTwoSided |= FlatMaterial.bTwoSided;

				if (FlatMaterial.DoesPropertyContainData(Property))
				{
					const FBox2D MaterialBox = InMaterialBoxes[MaterialIndex];
					const FIntPoint& InputSize = FlatMaterial.GetPropertySize(Property);
					TArray<FColor>& InputSamples = FlatMaterial.GetPropertySamples(Property);

					// Resize material to match output (area) size
					FIntPoint OutputSize = FIntPoint((OutTextureSize.X * MaterialBox.GetSize().X) - (InGutter * 2), (OutTextureSize.Y * MaterialBox.GetSize().Y) - (InGutter * 2));
					ConditionalImageResize(InputSize, OutputSize, InputSamples, false);

					// Copy material data to the merged 'atlas' texture
					FIntPoint OutputPosition = FIntPoint((OutTextureSize.X * MaterialBox.Min.X) + InGutter, (OutTextureSize.Y * MaterialBox.Min.Y) + InGutter);
					CopyTextureRect(InputSamples.GetData(), OutputSize, OutSamples.GetData(), OutTextureSize, OutputPosition, bCopyOnlyMaskedPixels);

					// Set the UV tranforms only once
					if (Index == 0)
					{
						FUVOffsetScalePair& UVTransform = OutUVTransforms[MaterialIndex];
						UVTransform.Key = MaterialBox.Min + (Gutter2D / FVector2D(OutTextureSize));
						UVTransform.Value = MaterialBox.GetSize() - ((Gutter2D * 2.0f) / FVector2D(OutTextureSize));
					}

					bMaterialsWritten = true;
				}
			}

			if (!bMaterialsWritten)
			{
				OutSamples.Empty();
				OutMergedMaterial.SetPropertySize(Property, FIntPoint(0, 0));
			}
			else
			{
				// Smear borders
				const FIntPoint PropertySize = OutMergedMaterial.GetPropertySize(Property);
				FMaterialBakingHelpers::PerformUVBorderSmear(OutSamples, PropertySize.X, PropertySize.Y);
			}
		}
	}
}


float FMeshMergeUtilities::FlattenEmissivescale(TArray<struct FFlattenMaterial>& InMaterialList) const
{
	// Find maximum emissive scaling value across materials
	float MaxScale = 0.0f;
	for (const FFlattenMaterial& Material : InMaterialList)
	{
		MaxScale = FMath::Max(MaxScale, Material.EmissiveScale);
	}
	
	// Renormalize samples 
	const float Multiplier = 1.0f / MaxScale;
	const int32 NumThreads = [&]()
	{
		return FPlatformProcess::SupportsMultithreading() ? FPlatformMisc::NumberOfCores() : 1;
	}();

	const int32 MaterialsPerThread = FMath::CeilToInt((float)InMaterialList.Num() / (float)NumThreads);
	ParallelFor(NumThreads, [&InMaterialList, MaterialsPerThread, Multiplier, MaxScale]
	(int32 Index)
	{
		int32 StartIndex = FMath::CeilToInt((float)Index * (float)MaterialsPerThread);
		const int32 EndIndex = FMath::Min(FMath::CeilToInt((float)(Index + 1) * (float)MaterialsPerThread), InMaterialList.Num());

		for (; StartIndex < EndIndex; ++StartIndex)
		{
			FFlattenMaterial& Material = InMaterialList[StartIndex];
			if (Material.EmissiveScale != MaxScale)
			{
				for (FColor& Sample : Material.GetPropertySamples(EFlattenMaterialProperties::Emissive))
				{
					if (Sample != FColor::Magenta)
					{
						Sample.R = Sample.R * Multiplier;
						Sample.G = Sample.G * Multiplier;
						Sample.B = Sample.B * Multiplier;
						Sample.A = Sample.A * Multiplier;
					}
				}
			}
		}
	}, NumThreads == 1);

	return MaxScale;
}

static TArray<FVector2D> GetCustomTextureCoordinates(const FMeshDescription& InMeshDescription, const UStaticMesh* InStaticMesh, const FMeshProxySettings& InMeshProxySettings)
{
	TArray<FVector2D> CustomTextureCoordinates;

	TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = FStaticMeshConstAttributes(InMeshDescription).GetVertexInstanceUVs();

	// If we already have lightmap uvs generated and they are valid, we can reuse those instead of having to generate new ones
	const int32 LightMapCoordinateIndex = InStaticMesh->GetLightMapCoordinateIndex();
	if (InMeshProxySettings.bReuseMeshLightmapUVs &&
		LightMapCoordinateIndex > 0 &&
		VertexInstanceUVs.GetNumElements() > 0 &&
		VertexInstanceUVs.GetNumChannels() > LightMapCoordinateIndex)
	{
		for (const FVertexInstanceID VertexInstanceID : InMeshDescription.VertexInstances().GetElementIDs())
		{
			CustomTextureCoordinates.Add(FVector2D(VertexInstanceUVs.Get(VertexInstanceID, LightMapCoordinateIndex)));
		}
	}
	else
	{
		FStaticMeshOperations::FGenerateUVOptions GenerateUVOptions;
		GenerateUVOptions.TextureResolution = InMeshProxySettings.MaterialSettings.TextureSize.GetMax();
		GenerateUVOptions.bMergeTrianglesWithIdenticalAttributes = false;
		GenerateUVOptions.UVMethod = GetUVGenerationMethodToUse();

		bool bSuccess = FStaticMeshOperations::GenerateUV(InMeshDescription, GenerateUVOptions, CustomTextureCoordinates);
		if (!bSuccess)
		{
			UE_LOGF(LogMeshMerging, Warning, "GenerateUV: Failed to pack UVs for static mesh \"%ls\" (num triangles = %d, texture resolution = %d).", *InStaticMesh->GetName(), InMeshDescription.Triangles().Num(), InMeshProxySettings.MaterialSettings.TextureSize.GetMax());
			CustomTextureCoordinates.Empty();
		}
	}

	return CustomTextureCoordinates;
}

class FProxyMeshDescriptor
{
public:
	FProxyMeshDescriptor(const UStaticMeshComponent* StaticMeshComponent, int32 LODIndex)
		: LODIndex(LODIndex)
		, LightMapIndex(INDEX_NONE)
	{
		ISMDescriptor.InitFrom(StaticMeshComponent, false);

		// Retrieve lightmap for usage of lightmap data
		if (StaticMeshComponent->LODData.IsValidIndex(0))
		{
			const FStaticMeshComponentLODInfo& ComponentLODInfo = StaticMeshComponent->LODData[0];
			const FMeshMapBuildData* MeshMapBuildData = StaticMeshComponent->GetMeshMapBuildData(ComponentLODInfo);
			if (MeshMapBuildData)
			{
				LightMap = MeshMapBuildData->LightMap;
				LightMapIndex = StaticMeshComponent->GetStaticMesh()->GetLightMapCoordinateIndex();
			}
		}
		
		Hash = ISMDescriptor.ComputeHash();
		Hash = FCrc::TypeCrc32(LODIndex, Hash);

		if (LightMapIndex != INDEX_NONE)
		{
			Hash = FCrc::TypeCrc32(LightMap.GetReference(), Hash);
			Hash = FCrc::TypeCrc32(LightMapIndex, Hash);
		}
	}

	bool operator==(const FProxyMeshDescriptor& InOther) const
	{
		return Hash == InOther.Hash &&
			   LODIndex == InOther.LODIndex &&
			   LightMap == InOther.LightMap &&
			   LightMapIndex == InOther.LightMapIndex &&
			   ISMDescriptor == InOther.ISMDescriptor;
	}

	int32 GetLODIndex() const { return LODIndex; }
	FLightMapRef GetLightMap() const { return LightMap; }
	int32 GetLightMapIndex() const { return LightMapIndex; }

	UStaticMesh* GetStaticMesh() const { return ISMDescriptor.StaticMesh; }

	bool IsValid() const { return !MeshDescription.IsEmpty(); }

	const FMeshDescription& GetMeshDescription() const
	{
		return MeshDescription;
	}

	const TArray<FVector2D>& GetCustomTextureCoordinates() const
	{
		return CustomTextureCoordinates;
	}

	void PrepareMeshDescription(const FMeshProxySettings& InMeshProxySettings)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PrepareMeshDescription);

		// Retrieve mesh data in FMeshDescription form
		FStaticMeshAttributes(MeshDescription).Register();
		FMeshMergeHelpers::RetrieveMesh(ISMDescriptor.StaticMesh, LODIndex, MeshDescription);

		CustomTextureCoordinates = ::GetCustomTextureCoordinates(MeshDescription, ISMDescriptor.StaticMesh, InMeshProxySettings);
		if (CustomTextureCoordinates.IsEmpty())
		{
			// Failure, clear mesh description
			MeshDescription.Empty();
		}
	}

private:
	int32			Hash;

	int32			LODIndex;
	FLightMapRef	LightMap;
	int32			LightMapIndex;

	FISMComponentDescriptor ISMDescriptor;

	FMeshDescription MeshDescription;
	TArray<FVector2D> CustomTextureCoordinates;
};

static void ScaleTextureCoordinatesToBox(const FBox2D& Box, TArray<FVector2D>& InOutTextureCoordinates)
{
	const FBox2D CoordinateBox(InOutTextureCoordinates);
	const FVector2D CoordinateRange = CoordinateBox.GetSize();
	const FVector2D Offset = CoordinateBox.Min + Box.Min;
	const FVector2D Scale = Box.GetSize() / CoordinateRange;
	for (FVector2D& Coordinate : InOutTextureCoordinates)
	{
		Coordinate = (Coordinate - Offset) * Scale;
	}
}

typedef TFunctionRef<int32(const UStaticMeshComponent*)> FGetMeshLODFunc;

struct FInstancedMeshDescriptionData
{
	FMeshDescription* MeshDescription;
	TArray<FTransform> InstancesTransforms;
};

static TArray<FInstancedMeshDescriptionData> GatherGeometry(const TArray<UStaticMeshComponent*>& InStaticMeshComponents, const FMeshProxySettings& InMeshProxySettings, TArray<FProxyMeshDescriptor>& InDescriptors, const TArray<TArray<int32>>& InMeshesToMergePerDescriptor, const TArray<int32>& InMeshesToMeshDescriptor, FGetMeshLODFunc InGetMeshLODFunc, int32& OutSummedLightmapPixels)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMergeUtilities::GatherGeometry);

	TArray<FMeshDescription> TempDescriptionData;
	TempDescriptionData.SetNum(InStaticMeshComponents.Num());

	TAtomic<int32>  SummedLightmapPixels(0);

	TArray<FInstancedMeshDescriptionData> MeshesDescriptions;
	MeshesDescriptions.SetNum(InDescriptors.Num());

	// If grouping by identical meshes, prepare each mesh description along with their flattened UVs
	// These meshes descriptions will serve for material baking, but also as the basis for creating a
	// single merged mesh out of all instances.
	if (InMeshProxySettings.bGroupIdenticalMeshesForBaking)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UniqueUVs);
		ParallelFor(InDescriptors.Num(), [&InDescriptors, &InMeshProxySettings](uint32 Index)
		{
			InDescriptors[Index].PrepareMeshDescription(InMeshProxySettings);
		});
	}

	// Gather geometry from each component, expand ISMC geometry
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GatherGeometryFromComponents);
		ParallelFor(InStaticMeshComponents.Num(), [InStaticMeshComponents, &InGetMeshLODFunc, InDescriptors, InMeshesToMeshDescriptor, InMeshProxySettings, &TempDescriptionData, &SummedLightmapPixels](uint32 Index)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GatherGeometryFromComponent);
			const UStaticMeshComponent* StaticMeshComponent = InStaticMeshComponents[Index];

			FMeshDescription& MeshDescription = TempDescriptionData[Index];

			// Retrieve meshes
			if (!InMeshProxySettings.bGroupIdenticalMeshesForBaking || !InDescriptors[InMeshesToMeshDescriptor[Index]].IsValid())
			{
				const int32 LODIndex = InGetMeshLODFunc(StaticMeshComponent);
				static const bool bPropagateVertexColours = true;

				// Retrieve mesh data in FMeshDescription form
				FStaticMeshAttributes(MeshDescription).Register();
				FMeshMergeHelpers::RetrieveMesh(StaticMeshComponent, LODIndex, MeshDescription, bPropagateVertexColours);
			}
			else
			{
				MeshDescription = InDescriptors[InMeshesToMeshDescriptor[Index]].GetMeshDescription();

				if (!InMeshProxySettings.bGroupIdenticalMeshesForBaking)
				{
					FStaticMeshOperations::ApplyTransform(MeshDescription, StaticMeshComponent->GetComponentTransform());
				}
			}

			// If the component is an ISMC then we need to duplicate the vertex data
			int32 NumInstances = 1;
			if (const UInstancedStaticMeshComponent* InstancedStaticMeshComponent = Cast<UInstancedStaticMeshComponent>(StaticMeshComponent))
			{
				if (!InMeshProxySettings.bGroupIdenticalMeshesForBaking)
				{
					FMeshMergeHelpers::ExpandInstances(InstancedStaticMeshComponent, MeshDescription);
				}

				NumInstances = InstancedStaticMeshComponent->PerInstanceSMData.Num();
			}

			int32 LightMapWidth, LightMapHeight;
			StaticMeshComponent->GetLightMapResolution(LightMapWidth, LightMapHeight);
			// Make sure we at least have some lightmap space allocated in case the static mesh is set up with invalid input
			SummedLightmapPixels += FMath::Max(16, LightMapHeight * LightMapWidth * NumInstances);
		}, EParallelForFlags::Unbalanced);
	}

	// For each mesh, append each component geometry
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AppendMeshes);
		FStaticMeshOperations::FAppendSettings AppendSettings;
		for (int32 ChannelIdx = 0; ChannelIdx < FStaticMeshOperations::FAppendSettings::MAX_NUM_UV_CHANNELS; ++ChannelIdx)
		{
			AppendSettings.bMergeUVChannels[ChannelIdx] = true;
		}

		ParallelFor(InDescriptors.Num(), [&MeshesDescriptions, &InMeshesToMergePerDescriptor, &InStaticMeshComponents, &TempDescriptionData, &AppendSettings, &InMeshProxySettings](uint32 Index)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AppendMeshes);
			FMeshDescription* TargetMeshDescription = new FMeshDescription();
			FStaticMeshAttributes(*TargetMeshDescription).Register();

			// When using this option, do not expand the instances, but rather send their transforms to the ProxyLOD tool
			if (InMeshProxySettings.bGroupIdenticalMeshesForBaking)
			{
				TArray<FTransform> InstancesTransforms;

				for (int32 Idx : InMeshesToMergePerDescriptor[Index])
				{
					UStaticMeshComponent* StaticMeshComponent = InStaticMeshComponents[Idx];

					if (InMeshProxySettings.bGroupIdenticalMeshesForBaking)
					{
						FTransform ComponentTransform = StaticMeshComponent->GetComponentTransform();

						if (const UInstancedStaticMeshComponent* InstancedStaticMeshComponent = Cast<UInstancedStaticMeshComponent>(StaticMeshComponent))
						{
							for (const FInstancedStaticMeshInstanceData& InstanceData : InstancedStaticMeshComponent->PerInstanceSMData)
							{
								InstancesTransforms.Add(FTransform(InstanceData.Transform) * ComponentTransform);
							}
						}
						else
						{
							InstancesTransforms.Add(ComponentTransform);
						}
					}
				}

				if (!InMeshesToMergePerDescriptor[Index].IsEmpty())
				{
					*TargetMeshDescription = TempDescriptionData[InMeshesToMergePerDescriptor[Index][0]];
				}

				MeshesDescriptions[Index].InstancesTransforms = MoveTemp(InstancesTransforms);
			}
			else
			{
				TArray<const FMeshDescription*> SourceMeshDescriptions;
				SourceMeshDescriptions.Reserve(InMeshesToMergePerDescriptor[Index].Num());
				for (int32 TempIdx : InMeshesToMergePerDescriptor[Index])
				{
					SourceMeshDescriptions.Add(&TempDescriptionData[TempIdx]);
				}

				FStaticMeshOperations::AppendMeshDescriptions(SourceMeshDescriptions, *TargetMeshDescription, AppendSettings);
			}

			MeshesDescriptions[Index].MeshDescription = TargetMeshDescription;
		}, EParallelForFlags::Unbalanced);
	}

	OutSummedLightmapPixels = SummedLightmapPixels;

	return MeshesDescriptions;
}

TArray<FMeshData> PrepareBakingMeshes(const struct FMeshProxySettings& InMeshProxySettings, const TArray<FProxyMeshDescriptor>& InDescriptors, TArray<FInstancedMeshDescriptionData> InMeshDescriptionData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PrepareBakingMeshes)

	check(InDescriptors.Num() == InMeshDescriptionData.Num());

	TArray<FMeshData> MeshData;
	MeshData.SetNum(InDescriptors.Num());

	// GetLightMap() must be called from the game thread
	check(IsInGameThread());
	for (int32 MeshIndex = 0; MeshIndex < InDescriptors.Num(); ++MeshIndex)
	{
		if (InDescriptors[MeshIndex].GetLightMapIndex() != INDEX_NONE)
		{
			MeshData[MeshIndex].LightMap = InDescriptors[MeshIndex].GetLightMap();
			MeshData[MeshIndex].LightMapIndex = InDescriptors[MeshIndex].GetLightMapIndex();
		}
	}

	// Parallel step - fetching custom (unwrapped) texture coordinates is the slowest part here
	ParallelFor(InDescriptors.Num(), [&MeshData, &InDescriptors, &InMeshDescriptionData, &InMeshProxySettings](uint32 MeshIndex)
	{
		const FProxyMeshDescriptor& MeshDescriptor = InDescriptors[MeshIndex];

		FMeshData& MeshSettings = MeshData[MeshIndex];
		MeshSettings.TextureCoordinateBox = FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f));

		if (InMeshProxySettings.bGroupIdenticalMeshesForBaking)
		{
			// Grouping by identical meshes, the UVs should have already been setup
			MeshSettings.MeshDescription = &MeshDescriptor.GetMeshDescription();
			MeshSettings.CustomTextureCoordinates = MeshDescriptor.GetCustomTextureCoordinates();
		}
		else
		{
			FMeshDescription& MeshDescription = *InMeshDescriptionData[MeshIndex].MeshDescription;
			MeshSettings.MeshDescription = &MeshDescription;
			MeshSettings.CustomTextureCoordinates = GetCustomTextureCoordinates(MeshDescription, MeshDescriptor.GetStaticMesh(), InMeshProxySettings);
		}

		if (MeshSettings.CustomTextureCoordinates.IsEmpty())
		{
			MeshSettings.MeshDescription = nullptr;
			MeshSettings.TextureCoordinateIndex = 0;
		}
	});

	return MeshData;
}

void FMeshMergeUtilities::CreateProxyMesh(const TArray<AActor*>& InActors, const struct FMeshProxySettings& InMeshProxySettings, UPackage* InOuter, const FString& InProxyBasePackageName, const FGuid InGuid, const FCreateProxyDelegate& InProxyCreatedDelegate, const bool bAllowAsync, const float ScreenSize) const
{
	CreateProxyMesh(InActors, InMeshProxySettings, GEngine->DefaultFlattenMaterial, InOuter, InProxyBasePackageName, InGuid, InProxyCreatedDelegate, bAllowAsync, ScreenSize);
}

void FMeshMergeUtilities::CreateProxyMesh(const TArray<UStaticMeshComponent*>& InStaticMeshComps, const struct FMeshProxySettings& InMeshProxySettings, UPackage* InOuter, const FString& InProxyBasePackageName, const FGuid InGuid, const FCreateProxyDelegate& InProxyCreatedDelegate, const bool bAllowAsync, const float ScreenSize) const
{
	CreateProxyMesh(InStaticMeshComps, InMeshProxySettings, GEngine->DefaultFlattenMaterial, InOuter, InProxyBasePackageName, InGuid, InProxyCreatedDelegate, bAllowAsync, ScreenSize);
}

void FMeshMergeUtilities::CreateProxyMesh(const TArray<AActor*>& InActors, const struct FMeshProxySettings& InMeshProxySettings, UMaterialInterface* InBaseMaterial, UPackage* InOuter, const FString& InProxyBasePackageName, const FGuid InGuid, const FCreateProxyDelegate& InProxyCreatedDelegate, const bool bAllowAsync /*= false*/, const float ScreenSize /*= 1.0f*/) const
{
	// No actors given as input
	if (InActors.Num() == 0)
	{
		UE_LOGF(LogMeshMerging, Log, "No actors specified to generate a proxy mesh for");
		return;
	}

	// Collect components to merge
	TArray<UStaticMeshComponent*> ComponentsToMerge;
	for (AActor* Actor : InActors)
	{
		TInlineComponentArray<UStaticMeshComponent*> Components;
		Actor->GetComponents(Components);
		ComponentsToMerge.Append(Components);
	}

	CreateProxyMesh(ComponentsToMerge, InMeshProxySettings, InBaseMaterial, InOuter, InProxyBasePackageName, InGuid, InProxyCreatedDelegate, bAllowAsync, ScreenSize);
}

void FMeshMergeUtilities::CreateProxyMesh(const TArray<UStaticMeshComponent*>& InComponentsToMerge, const struct FMeshProxySettings& InMeshProxySettings, UMaterialInterface* InBaseMaterial,
	UPackage* InOuter, const FString& InProxyBasePackageName, const FGuid InGuid, const FCreateProxyDelegate& InProxyCreatedDelegate, const bool bAllowAsync, const float ScreenSize) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMergeUtilities::CreateProxyMesh)

	// The MeshReductionInterface manages the choice mesh reduction plugins, Unreal native vs third party (e.g. Simplygon)
	IMeshReductionModule& ReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionModule>("MeshReductionInterface");

	// Error/warning checking for input
	if (ReductionModule.GetMeshMergingInterface() == nullptr)
	{
		UE_LOGF(LogMeshMerging, Error, "No mesh reduction module available. You must enable a plugin that provides that functionality (ex: ProxyLODPlugin)");
		return;
	}

	// Check that the delegate has a func-ptr bound to it
	if (!InProxyCreatedDelegate.IsBound())
	{
		UE_LOGF(LogMeshMerging, Warning, "Invalid (unbound) delegate for returning generated proxy mesh");
		return;
	}

	TArray<UStaticMeshComponent*> ComponentsToMerge = InComponentsToMerge;

	// Remove invalid components
	ComponentsToMerge.RemoveAll([](UStaticMeshComponent* Val) { return Val->GetStaticMesh() == nullptr; });

	// No actors given as input
	if (ComponentsToMerge.Num() == 0)
	{
		UE_LOGF(LogMeshMerging, Log, "No static mesh specified to generate a proxy mesh for");
		
		TArray<UObject*> OutAssetsToSync;
		InProxyCreatedDelegate.ExecuteIfBound(InGuid, OutAssetsToSync);

		return;
	}

	// Base asset name for a new assets
	// In case outer is null ProxyBasePackageName has to be long package name
	if (InOuter == nullptr && FPackageName::IsShortPackageName(InProxyBasePackageName))
	{
		UE_LOGF(LogMeshMerging, Warning, "Invalid long package name: '%ls'.", *InProxyBasePackageName);
		return;
	}

	FScopedSlowTask SlowTask(100.f, (LOCTEXT("CreateProxyMesh_CreateMesh", "Creating Mesh Proxy")));
	SlowTask.MakeDialog();

	TArray<FRawMeshExt> SourceMeshes;
	TMap<FMeshIdAndLOD, TArray<int32>> GlobalMaterialMap;

	FBoxSphereBounds::Builder EstimatedBoundsBuilder;
	for (const UStaticMeshComponent* StaticMeshComponent : ComponentsToMerge)
	{
		EstimatedBoundsBuilder += StaticMeshComponent->Bounds;
	}
	FBoxSphereBounds EstimatedBounds = EstimatedBoundsBuilder;

	static const float FOVRad = FMath::DegreesToRadians(45.0f);
	static const FMatrix ProjectionMatrix = FPerspectiveMatrix(FOVRad, 1920, 1080, 0.01f);
	FHierarchicalLODUtilitiesModule& HLODModule = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
	IHierarchicalLODUtilities* Utilities = HLODModule.GetUtilities();
	float EstimatedDistance = Utilities->CalculateDrawDistanceFromScreenSize(EstimatedBounds.SphereRadius, ScreenSize, ProjectionMatrix);

	auto SelectLODFunc = [&InMeshProxySettings, &Utilities, EstimatedDistance] (const UStaticMeshComponent* Component)
	{
		int32 LODIndex = 0;
		if (InMeshProxySettings.bCalculateCorrectLODModel)
		{
			LODIndex = Utilities->GetLODLevelForScreenSize(Component, Utilities->CalculateScreenSizeFromDrawDistance(Component->Bounds.SphereRadius, ProjectionMatrix, EstimatedDistance));
		}
		return LODIndex;
	};

	SlowTask.EnterProgressFrame(5.0f, LOCTEXT("CreateProxyMesh_CollectingMeshes", "Collecting Input Static Meshes"));

	// Mesh / LOD index	
	TMap<uint32, FMeshDescription*> RawMeshLODs;

	// Mesh index, <original section index, unique section index>
	TMultiMap<uint32, TPair<uint32, uint32>> MeshSectionToUniqueSection;

	// Unique set of sections in mesh
	TArray<FSectionInfo> UniqueSections;
	TMultiMap<uint32, uint32> SectionToMesh;

	TArray<const UStaticMeshComponent*> ImposterMeshComponents;
	TArray<UStaticMeshComponent*> StaticMeshComponents;
	for (UStaticMeshComponent* StaticMeshComponent : ComponentsToMerge)
	{
		if (StaticMeshComponent->HLODBatchingPolicy != EHLODBatchingPolicy::None)
		{
			ImposterMeshComponents.Add(StaticMeshComponent);
		}
		else
		{
			StaticMeshComponents.Add(StaticMeshComponent);
		}
	}

	TArray<FProxyMeshDescriptor>	MeshDescriptors;
	TArray<TArray<int32>>			MeshesToMergePerDescriptor;
	TArray<int32>					MeshToMeshDescriptor;

	TArray<TArray<FSectionInfo>>	GlobalSections;

	MeshToMeshDescriptor.Reserve(StaticMeshComponents.Num());

	for (int32 ComponentIndex = 0; ComponentIndex < StaticMeshComponents.Num(); ++ComponentIndex)
	{
		const UStaticMeshComponent* StaticMeshComponent = StaticMeshComponents[ComponentIndex];

		FProxyMeshDescriptor MeshDescriptor(StaticMeshComponent, SelectLODFunc(StaticMeshComponent));

		int32 Index;
		if (!InMeshProxySettings.bGroupIdenticalMeshesForBaking || !MeshDescriptors.Find(MeshDescriptor, Index))
		{
			Index = MeshDescriptors.Num();
			MeshDescriptors.Add(MeshDescriptor);
			MeshesToMergePerDescriptor.AddDefaulted();

			TArray<FSectionInfo>& Sections = GlobalSections.AddDefaulted_GetRef();

			// Extract sections for given LOD index from the mesh 
			FMeshMergeHelpers::ExtractSections(StaticMeshComponent, MeshDescriptor.GetLODIndex(), Sections);
		}

		MeshesToMergePerDescriptor[Index].Add(ComponentIndex);
		MeshToMeshDescriptor.Add(Index);
	}

	for (int32 MeshIndex = 0; MeshIndex < GlobalSections.Num(); ++MeshIndex)
	{
		TArray<FSectionInfo>& Sections = GlobalSections[MeshIndex];

		for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
		{
			FSectionInfo& Section = Sections[SectionIndex];

			const int32 UniqueIndex = UniqueSections.AddUnique(Section);
			MeshSectionToUniqueSection.Add(MeshIndex, TPair<uint32, uint32>(SectionIndex, UniqueIndex));
			SectionToMesh.Add(UniqueIndex, MeshIndex);
		}
	}

	int32 SummedLightmapPixels;
	
	TArray<FInstancedMeshDescriptionData> MeshDescriptionData = GatherGeometry(StaticMeshComponents, InMeshProxySettings, MeshDescriptors, MeshesToMergePerDescriptor, MeshToMeshDescriptor, SelectLODFunc, SummedLightmapPixels);
	TArray<FMeshData> MeshBakingData = PrepareBakingMeshes(InMeshProxySettings, MeshDescriptors, MeshDescriptionData);

	TArray<UMaterialInterface*> UniqueMaterials;
	//Unique material index to unique section index
	TMultiMap<uint32, uint32> MaterialToSectionMap;
	for (int32 SectionIndex = 0; SectionIndex < UniqueSections.Num(); ++SectionIndex)
	{
		FSectionInfo& Section = UniqueSections[SectionIndex];
		const int32 UniqueIndex = UniqueMaterials.AddUnique(Section.Material);
		MaterialToSectionMap.Add(UniqueIndex, SectionIndex);
	}

	TArray<FMeshData> GlobalMeshSettings;
	TArray<FMaterialData> GlobalMaterialSettings;

	FMaterialProxySettings MaterialProxySettings = InMeshProxySettings.MaterialSettings;
	TArray<UPrimitiveComponent*> PrimitiveComponents;
	Algo::Transform(StaticMeshComponents, PrimitiveComponents, [](UStaticMeshComponent* SMComponent) { return SMComponent; });
	if (MaterialProxySettings.ResolveTexelDensity(PrimitiveComponents))
	{
		double Total3DArea = 0;

		for (const FInstancedMeshDescriptionData& InstancedMeshDescriptionData : MeshDescriptionData)
		{
			double Mesh3DArea = 0;

			const FMeshDescription& MeshDescription = *InstancedMeshDescriptionData.MeshDescription;

			FStaticMeshConstAttributes Attributes(MeshDescription);
			TVertexAttributesConstRef<FVector3f> Positions = Attributes.GetVertexPositions();

			for (const FTriangleID TriangleID : MeshDescription.Triangles().GetElementIDs())
			{
				// World space area
				TArrayView<const FVertexID> TriVertices = MeshDescription.GetTriangleVertices(TriangleID);
				Mesh3DArea += UE::Geometry::VectorUtil::Area(Positions[TriVertices[0]], Positions[TriVertices[1]], Positions[TriVertices[2]]);
			}

			// Account for multiple instances (no transforms means a single instance)
			uint32 NumInstances = FMath::Max(1, InstancedMeshDescriptionData.InstancesTransforms.Num());
			Total3DArea += Mesh3DArea * NumInstances;
		}

		MaterialProxySettings.TextureSize = FMaterialUtilities::GetTextureSizeFromTargetTexelDensity(Total3DArea, 1.0f, MaterialProxySettings.TargetTexelDensityPerMeter);
		MaterialProxySettings.TextureSizingType = ETextureSizingType::TextureSizingType_UseSingleTextureSize;
	}

	UMaterialOptions* Options = PopulateMaterialOptions(MaterialProxySettings);
	TGCObjectScopeGuard<UMaterialOptions> MaterialOptionsGCScopeGuard(Options);

	TArray<EMaterialProperty> MaterialProperties;
	for (const FPropertyEntry& Entry : Options->Properties)
	{
		if (Entry.Property != MP_MAX)
		{
			MaterialProperties.Add(Entry.Property);
		}
	}

	// Mesh index / ( Mesh relative section index / output index )	
	TMultiMap<uint32, TPair<uint32, uint32>> OutputMaterialsMap;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMergeUtilities::MaterialAnalysisAndUVGathering);

		for (int32 MaterialIndex = 0; MaterialIndex < UniqueMaterials.Num(); ++MaterialIndex)
		{
			UMaterialInterface* Material = UniqueMaterials[MaterialIndex];

			//Unique section indices
			TArray<uint32> SectionIndices;
			MaterialToSectionMap.MultiFind(MaterialIndex, SectionIndices);

			// Check whether or not this material requires mesh data
			int32 NumTexCoords = 0;
			bool bUseVertexData = false;
			FMaterialUtilities::AnalyzeMaterial(Material, MaterialProperties, NumTexCoords, bUseVertexData);

			FMaterialData MaterialSettings;
			MaterialSettings.Material = Material;

			for (const FPropertyEntry& Entry : Options->Properties)
			{
				// With substrate, we cannot use connection on the root node to infer Active or Constant properties. So we activate all material input for now. 
				// SUBSTRATE_TODO: gather information about the material used input, maybe using Material->GetSubstrateCompilationConfig().
				const float bIsSubstrateEnabled = Substrate::IsSubstrateEnabled();

				if ((bIsSubstrateEnabled || (!Entry.bUseConstantValue && Material->IsPropertyActive(Entry.Property))) && Entry.Property != MP_MAX)
				{
					MaterialSettings.PropertySizes.Add(Entry.Property, Entry.bUseCustomSize ? Entry.CustomSize : Options->TextureSize);
				}
			}

			if (bUseVertexData || NumTexCoords != 0)
			{
				for (uint32 SectionIndex : SectionIndices)
				{
					TArray<uint32> MeshIndices;
					SectionToMesh.MultiFind(SectionIndex, MeshIndices);

					for (const uint32 MeshIndex : MeshIndices)
					{
						FMeshData MeshSettings = MeshBakingData[MeshIndex];

						// Section index is a unique one so we need to map it to the mesh's equivalent(s)
						TArray<TPair<uint32, uint32>> SectionToUniqueSectionIndices;
						MeshSectionToUniqueSection.MultiFind(MeshIndex, SectionToUniqueSectionIndices);
						for (const TPair<uint32, uint32>& IndexPair : SectionToUniqueSectionIndices)
						{
							if (IndexPair.Value == SectionIndex)
							{
								MeshSettings.MaterialIndices.Add(IndexPair.Key);
								OutputMaterialsMap.Add(MeshIndex, TPair<uint32, uint32>(IndexPair.Key, GlobalMeshSettings.Num()));
							}
						}

						GlobalMeshSettings.Add(MoveTemp(MeshSettings));
						GlobalMaterialSettings.Add(MaterialSettings);
					}
				}
			}
			else
			{
				// Add simple bake entry 
				FMeshData MeshSettings;
				MeshSettings.MeshDescription = nullptr;
				MeshSettings.TextureCoordinateBox = FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f));
				MeshSettings.TextureCoordinateIndex = 0;

				// For each original material index add an entry to the corresponding LOD and bake output index 
				for (uint32 SectionIndex : SectionIndices)
				{
					TArray<uint32> MeshIndices;
					SectionToMesh.MultiFind(SectionIndex, MeshIndices);

					for (uint32 MeshIndex : MeshIndices)
					{
						TArray<TPair<uint32, uint32>> SectionToUniqueSectionIndices;
						MeshSectionToUniqueSection.MultiFind(MeshIndex, SectionToUniqueSectionIndices);
						for (const TPair<uint32, uint32>& IndexPair : SectionToUniqueSectionIndices)
						{
							if (IndexPair.Value == SectionIndex)
							{
								OutputMaterialsMap.Add(MeshIndex, TPair<uint32, uint32>(IndexPair.Key, GlobalMeshSettings.Num()));
							}
						}
					}
				}
				
				GlobalMeshSettings.Add(MeshSettings);
				GlobalMaterialSettings.Add(MaterialSettings);
			}
		}
	}

	TArray<FFlattenMaterial> FlattenedMaterials;
	IMaterialBakingModule& MaterialBakingModule = FModuleManager::Get().LoadModuleChecked<IMaterialBakingModule>("MaterialBaking");

	auto MaterialFlattenLambda =
		[this, &Options, &GlobalMeshSettings, &GlobalMaterialSettings, &MeshDescriptionData, &OutputMaterialsMap, &MaterialBakingModule](TArray<FFlattenMaterial>& FlattenedMaterialArray)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MaterialFlatten)

		TArray<FMeshData*> MeshSettingPtrs;
		for (int32 SettingsIndex = 0; SettingsIndex < GlobalMeshSettings.Num(); ++SettingsIndex)
		{
			MeshSettingPtrs.Add(&GlobalMeshSettings[SettingsIndex]);
		}

		TArray<FMaterialData*> MaterialSettingPtrs;
		for (int32 SettingsIndex = 0; SettingsIndex < GlobalMaterialSettings.Num(); ++SettingsIndex)
		{
			MaterialSettingPtrs.Add(&GlobalMaterialSettings[SettingsIndex]);
		}

		// This scope ensures BakeOutputs is never used after TransferOutputToFlatMaterials
		{
			TArray<FBakeOutput> BakeOutputs;
			MaterialBakingModule.BakeMaterials(MaterialSettingPtrs, MeshSettingPtrs, BakeOutputs);

			// Append constant properties ?
			TArray<FColor> ConstantData;
			FIntPoint ConstantSize(1, 1);
			for (const FPropertyEntry& Entry : Options->Properties)
			{
				if (Entry.bUseConstantValue && Entry.Property != MP_MAX)
				{
					ConstantData.SetNum(1, EAllowShrinking::No);
					ConstantData[0] = FColor(Entry.ConstantValue * 255.0f, Entry.ConstantValue * 255.0f, Entry.ConstantValue * 255.0f);
					for (FBakeOutput& Output : BakeOutputs)
					{
						Output.PropertyData.Add(Entry.Property, ConstantData);
						Output.PropertySizes.Add(Entry.Property, ConstantSize);
					}
				}
			}

			TransferOutputToFlatMaterials(GlobalMaterialSettings, BakeOutputs, FlattenedMaterialArray);
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(RemapBakedMaterials)

			// Now have the baked out material data, need to have a map or actually remap the raw mesh data to baked material indices
			for (int32 MeshIndex = 0; MeshIndex < MeshDescriptionData.Num(); ++MeshIndex)
			{
				FMeshDescription& MeshDescription = *MeshDescriptionData[MeshIndex].MeshDescription;

				TArray<TPair<uint32, uint32>> SectionAndOutputIndices;
				OutputMaterialsMap.MultiFind(MeshIndex, SectionAndOutputIndices);
				TArray<int32> Remap;
				// Reorder loops
				for (const TPair<uint32, uint32>& IndexPair : SectionAndOutputIndices)
				{
					const int32 SectionIndex = IndexPair.Key;
					const int32 NewIndex = IndexPair.Value;

					if (Remap.Num() < (SectionIndex + 1))
					{
						Remap.SetNum(SectionIndex + 1);
					}

					Remap[SectionIndex] = NewIndex;
				}
			
				TMap<FPolygonGroupID, FPolygonGroupID> RemapPolygonGroup;
				for (const FPolygonGroupID PolygonGroupID : MeshDescription.PolygonGroups().GetElementIDs())
				{
					checkf(Remap.IsValidIndex(PolygonGroupID.GetValue()), TEXT("Missing material bake output index entry for mesh(section)"));
					int32 RemapID = Remap[PolygonGroupID.GetValue()];
					RemapPolygonGroup.Add(PolygonGroupID, FPolygonGroupID(RemapID));
				}
				MeshDescription.RemapPolygonGroups(RemapPolygonGroup);
			}
		}
	};

	// Landscape culling.  NB these are temporary copies of the culling data and should be deleted after use.
	TArray<FMeshDescription*> CullingRawMeshes;
	if (InMeshProxySettings.bUseLandscapeCulling)
	{
		SlowTask.EnterProgressFrame(5.0f, LOCTEXT("CreateProxyMesh_LandscapeCulling", "Applying Landscape Culling"));
		UWorld* InWorld = ComponentsToMerge[0]->GetWorld();
		FMeshMergeHelpers::RetrieveCullingLandscapeAndVolumes(InWorld, EstimatedBounds, InMeshProxySettings.LandscapeCullingPrecision, CullingRawMeshes);
	}

	// Allocate merge complete data
	FMergeCompleteData* Data = new FMergeCompleteData();
	Data->InOuter = InOuter;
	Data->InProxySettings = InMeshProxySettings;
	Data->ProxyBasePackageName = InProxyBasePackageName;
	Data->CallbackDelegate = InProxyCreatedDelegate;
	Data->ImposterComponents = ImposterMeshComponents;
	Data->StaticMeshComponents = StaticMeshComponents;
	Data->BaseMaterial = InBaseMaterial;

	// Lightmap resolution
	if (InMeshProxySettings.bComputeLightMapResolution)
	{
		Data->InProxySettings.LightMapResolution = FMath::CeilToInt(FMath::Sqrt(static_cast<float>(SummedLightmapPixels)));
	}

	// Add this proxy job to map	
	Processor->AddProxyJob(InGuid, Data);

	TArray<FInstancedMeshMergeData> MergeDataEntries;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MergeDataPreparation)

		for (int32 Index = 0; Index < MeshDescriptionData.Num(); ++Index)
		{
			FInstancedMeshMergeData MergeData;
			MergeData.SourceStaticMesh = MeshDescriptors[Index].GetStaticMesh();
			MergeData.RawMesh = MeshDescriptionData[Index].MeshDescription;
			MergeData.NewUVs = MeshDescriptors[Index].GetCustomTextureCoordinates();
			MergeData.bIsClippingMesh = false;
			MergeData.InstanceTransforms = MeshDescriptionData[Index].InstancesTransforms;

			FMeshMergeHelpers::CalculateTextureCoordinateBoundsForMesh(*MergeData.RawMesh, MergeData.TexCoordBounds);

			if (MergeData.NewUVs.IsEmpty())
			{
				FMeshData* MeshData = GlobalMeshSettings.FindByPredicate([&](const FMeshData& Entry)
				{
					return Entry.MeshDescription == MergeData.RawMesh && (Entry.CustomTextureCoordinates.Num() || Entry.TextureCoordinateIndex != 0);
				});

				if (MeshData)
				{
					if (MeshData->CustomTextureCoordinates.Num())
					{
						MergeData.NewUVs = MeshData->CustomTextureCoordinates;
					}
					else
					{
						TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = FStaticMeshConstAttributes(*MeshData->MeshDescription).GetVertexInstanceUVs();
						MergeData.NewUVs.Reset(MeshData->MeshDescription->VertexInstances().Num());
						for (const FVertexInstanceID VertexInstanceID : MeshData->MeshDescription->VertexInstances().GetElementIDs())
						{
							MergeData.NewUVs.Add(FVector2D(VertexInstanceUVs.Get(VertexInstanceID, MeshData->TextureCoordinateIndex)));
						}
					}
					MergeData.TexCoordBounds[0] = FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f));
				}
			}
			MergeDataEntries.Add(MergeData);
		}
	}

	if (MergeDataEntries.Num() != 0)
	{
		// Populate landscape clipping geometry
		for (FMeshDescription* RawMesh : CullingRawMeshes)
		{
			FInstancedMeshMergeData ClipData;
			ClipData.bIsClippingMesh = true;
			ClipData.RawMesh = RawMesh;
			MergeDataEntries.Add(ClipData);
		}

		SlowTask.EnterProgressFrame(50.0f, LOCTEXT("CreateProxyMesh_GenerateProxy", "Generating Proxy Mesh"));

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ProxyGeneration)

			// Choose Simplygon Swarm (if available) or local proxy lod method
			if (ReductionModule.GetDistributedMeshMergingInterface() != nullptr && GetDefault<UEditorPerProjectUserSettings>()->bUseSimplygonSwarm && bAllowAsync)
			{
				MaterialFlattenLambda(FlattenedMaterials);

				ReductionModule.GetDistributedMeshMergingInterface()->ProxyLOD(MergeDataEntries, Data->InProxySettings, FlattenedMaterials, InGuid);
			}
			else
			{
				IMeshMerging* MeshMerging = ReductionModule.GetMeshMergingInterface();

				// Register the Material Flattening code if parallel execution is supported, otherwise directly run it.

				if (MeshMerging->bSupportsParallelMaterialBake())
				{
					MeshMerging->BakeMaterialsDelegate.BindLambda(MaterialFlattenLambda);
				}
				else
				{
					MaterialFlattenLambda(FlattenedMaterials);
				}

				MeshMerging->ProxyLOD(MergeDataEntries, Data->InProxySettings, FlattenedMaterials, InGuid);


				Processor->Tick(0); // make sure caller gets merging results
			}
		}
	}
	else
	{
		FMeshDescription MeshDescription;
		FStaticMeshAttributes(MeshDescription).Register();
		FFlattenMaterial FlattenMaterial;
		Processor->ProxyGenerationComplete(MeshDescription, FlattenMaterial, InGuid);
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(Cleanup)

	// Clean up the CullingRawMeshes
	ParallelFor(CullingRawMeshes.Num(),
		[&CullingRawMeshes](int32 Index)
		{
			delete CullingRawMeshes[Index];
		}
	);

	// Clean up the MeshDescriptionData
	ParallelFor(
		MeshDescriptionData.Num(),
		[&MeshDescriptionData](int32 Index)
		{
			delete MeshDescriptionData[Index].MeshDescription;
		}
	);
}

void FMeshMergeUtilities::RetrieveMeshDescription(const UStaticMeshComponent* InStaticMeshComponent, int32 LODIndex, FMeshDescription& InOutMeshDescription, bool bPropagateMeshData) const
{
	FMeshMergeHelpers::RetrieveMesh(InStaticMeshComponent, LODIndex, InOutMeshDescription, bPropagateMeshData);
}

void FMeshMergeUtilities::RetrieveMeshDescription(const USkeletalMeshComponent* InSkeletalMeshComponent, int32 LODIndex, FMeshDescription& InOutMeshDescription, bool bPropagateMeshData) const
{
	FMeshMergeHelpers::RetrieveMesh(InSkeletalMeshComponent, LODIndex, InOutMeshDescription, bPropagateMeshData);
}

void FMeshMergeUtilities::RetrieveMeshDescription(const UStaticMesh* InStaticMesh, int32 LODIndex, FMeshDescription& InOutMeshDescription) const
{
	FMeshMergeHelpers::RetrieveMesh(InStaticMesh, LODIndex, InOutMeshDescription);
}

void FMeshMergeUtilities::RegisterExtension(IMeshMergeExtension* InExtension)
{
	MeshMergeExtensions.Add(InExtension);
}

void FMeshMergeUtilities::UnregisterExtension(IMeshMergeExtension* InExtension)
{
	MeshMergeExtensions.Remove(InExtension);
}

bool RetrieveRawMeshData(FMeshMergeDataTracker& DataTracker
	, const int32 ComponentIndex
	, const int32 LODIndex
	, UStaticMeshComponent* Component
	, const bool bPropagateMeshData
	, TArray<FSectionInfo>& Sections
	, FStaticMeshComponentAdapter& Adapter
	, const bool bMergeMaterialData
	, const FMeshMergingSettings& InSettings)
{
	// Retrieve raw mesh data
	FMeshDescription& RawMesh = DataTracker.AddAndRetrieveRawMesh(ComponentIndex, LODIndex, Component->GetStaticMesh());
	Adapter.RetrieveRawMeshData(LODIndex, RawMesh, bPropagateMeshData);

	// Reset section for reuse
	Sections.SetNum(0, EAllowShrinking::No);

	// Extract sections for given LOD index from the mesh 
	Adapter.RetrieveMeshSections(LODIndex, Sections);

	for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
	{
		const FSectionInfo& Section = Sections[SectionIndex];
		// Unique section index for remapping
		const int32 UniqueIndex = DataTracker.AddSection(Section);

		// Store of original to unique section index entry for this component + LOD index
		DataTracker.AddSectionRemapping(ComponentIndex, LODIndex, SectionIndex, UniqueIndex);
		DataTracker.AddMaterialSlotName(Section.Material, Section.MaterialSlotName);

		if (!bMergeMaterialData)
		{
			FStaticMeshOperations::SwapPolygonPolygonGroup(RawMesh, UniqueIndex, Section.StartIndex, Section.EndIndex, false);
		}
	}
	
	//Compact the PolygonGroupID to make sure it follow the section index
	FElementIDRemappings RemapInformation;
	RawMesh.Compact(RemapInformation);

	// If the component is an ISMC then we need to duplicate the vertex data
	if (Component->IsA<UInstancedStaticMeshComponent>())
	{
		const UInstancedStaticMeshComponent* InstancedStaticMeshComponent = Cast<UInstancedStaticMeshComponent>(Component);
		FMeshMergeHelpers::ExpandInstances(InstancedStaticMeshComponent, RawMesh);
	}

	if (InSettings.bUseLandscapeCulling)
	{
		FMeshMergeHelpers::CullTrianglesFromVolumesAndUnderLandscapes(Component->GetWorld(), Adapter.GetBounds(), RawMesh);
	}

	// If the valid became invalid during retrieval remove it again
	const bool bValidMesh = RawMesh.VertexInstances().Num() > 0;
	if (!bValidMesh)
	{
		DataTracker.RemoveRawMesh(ComponentIndex, LODIndex);
	}
	else if (Component->GetStaticMesh() != nullptr)
	{
		// If the mesh is valid at this point, record the lightmap UV so we have a record for use later
		DataTracker.AddLightmapChannelRecord(ComponentIndex, LODIndex, Component->GetStaticMesh()->GetLightMapCoordinateIndex());
	}
	return bValidMesh;
}

void FMeshMergeUtilities::MergeComponentsToStaticMesh(const TArray<UPrimitiveComponent*>& ComponentsToMerge, UWorld* World, const FMeshMergingSettings& InSettings, UMaterialInterface* InBaseMaterial, UPackage* InOuter, const FString& InBasePackageName, TArray<UObject*>& OutAssetsToSync, FVector& OutMergedActorLocation, const float ScreenSize, bool bSilent /*= false*/) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMergeUtilities::MergeComponentsToStaticMesh);

	// Use first mesh for naming and pivot
	bool bFirstMesh = true;
	FString MergedAssetPackageName;
	FVector MergedAssetPivot;
	
	TArray<UStaticMeshComponent*> StaticMeshComponentsToMerge;
	TArray<const UStaticMeshComponent*> ImposterComponents;

	for (int32 MeshId = 0; MeshId < ComponentsToMerge.Num(); ++MeshId)
	{
		UStaticMeshComponent* MeshComponent = Cast<UStaticMeshComponent>(ComponentsToMerge[MeshId]);
		if (MeshComponent && MeshComponent->GetStaticMesh())
		{
			// Make sure referenced lightmaps and shadowmaps are compiled
			if (MeshComponent->LODData.IsValidIndex(0))
			{
				const FStaticMeshComponentLODInfo& ComponentLODInfo = MeshComponent->LODData[0];
				const FMeshMapBuildData* MeshMapBuildData = MeshComponent->GetMeshMapBuildData(ComponentLODInfo);
				if (MeshMapBuildData)
				{
					TArray<UTexture2D*> ReferencedTextures;

					FLightMap2D* Lightmap = MeshMapBuildData && MeshMapBuildData->LightMap ? MeshMapBuildData->LightMap->GetLightMap2D() : nullptr;
					if (Lightmap)
					{
						Lightmap->GetReferencedTextures(ReferencedTextures);
					}

					FShadowMap2D* Shadowmap = MeshMapBuildData && MeshMapBuildData->ShadowMap ? MeshMapBuildData->ShadowMap->GetShadowMap2D() : nullptr;
					if (Shadowmap && Shadowmap->IsValid())
					{
						ReferencedTextures.Add(Shadowmap->GetTexture());
					}
					
					FTextureCompilingManager::Get().FinishCompilation(TArray<UTexture*>(MoveTemp(ReferencedTextures)));
				}
			}

			if((MeshComponent->HLODBatchingPolicy != EHLODBatchingPolicy::None) && InSettings.bIncludeImposters)
			{
				ImposterComponents.Add(MeshComponent);
			}
			else
			{
				StaticMeshComponentsToMerge.Add(MeshComponent);
			}

			// Save the pivot and asset package name of the first mesh, will later be used for creating merged mesh asset 
			if (bFirstMesh)
			{
				// Mesh component pivot point
				switch (InSettings.PivotType)
				{
				case EMeshMergePivotType::WorldOrigin:
					MergedAssetPivot = FVector::ZeroVector;
					break;

				case EMeshMergePivotType::Automatic:
					MergedAssetPivot = MeshComponent->GetComponentTransform().GetLocation();
					break;

				case EMeshMergePivotType::Custom:
					MergedAssetPivot = InSettings.CustomPivotLocation;
					break;

				default:
					checkNoEntry();
				}

				// Source mesh asset package name
				MergedAssetPackageName = MeshComponent->GetStaticMesh()->GetOutermost()->GetName();

				bFirstMesh = false;
			}
		}
	}

	// Nothing to do if no StaticMeshComponents
	if (StaticMeshComponentsToMerge.Num() == 0 && ImposterComponents.Num() == 0)
	{
		return;
	}

	FMeshMergeDataTracker DataTracker;

	const bool bMergeAllLODs = InSettings.LODSelectionType == EMeshLODSelectionType::AllLODs;
	const bool bMergeMaterialData = InSettings.bMergeMaterials && InSettings.LODSelectionType != EMeshLODSelectionType::AllLODs;
	const bool bPropagateMeshData = InSettings.bBakeVertexDataToMesh || (bMergeMaterialData && InSettings.bUseVertexDataForBakingMaterial);

	TArray<FStaticMeshComponentAdapter> Adapters;

	TArray<FSectionInfo> Sections;
	if (bMergeAllLODs)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RetrieveRawMeshData);
		for (int32 ComponentIndex = 0; ComponentIndex < StaticMeshComponentsToMerge.Num(); ++ComponentIndex)
		{
			UStaticMeshComponent* Component = StaticMeshComponentsToMerge[ComponentIndex];
			Adapters.Add(FStaticMeshComponentAdapter(Component));
			FStaticMeshComponentAdapter& Adapter = Adapters.Last();
			
			if (InSettings.bComputedLightMapResolution)
			{
				int32 LightMapHeight, LightMapWidth;
				if (Component->GetLightMapResolution(LightMapWidth, LightMapHeight))
				{
					DataTracker.AddLightMapPixels(LightMapWidth * LightMapHeight);
				}
			}			
						
			const int32 NumLODs = [&]()
			{
				const int32 NumberOfLODsAvailable = Adapter.GetNumberOfLODs();
				if (Component->HLODBatchingPolicy != EHLODBatchingPolicy::None)
				{
					return InSettings.bIncludeImposters ? NumberOfLODsAvailable : NumberOfLODsAvailable - 1;
				}

				return NumberOfLODsAvailable;
			}();

			for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
			{
				if (!RetrieveRawMeshData(DataTracker
					, ComponentIndex
					, LODIndex
					, Component
					, bPropagateMeshData
					, Sections
					, Adapter
					, false
					, InSettings))
				{
					//If the rawmesh was not retrieve properly break the loop
					break;
				}
				DataTracker.AddLODIndex(LODIndex);
			}
		}
	}
	else
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RetrieveRawMeshData);

		// Retrieve HLOD module for calculating LOD index from screen size
		FHierarchicalLODUtilitiesModule& Module = FModuleManager::LoadModuleChecked<FHierarchicalLODUtilitiesModule>("HierarchicalLODUtilities");
		IHierarchicalLODUtilities* Utilities = Module.GetUtilities();

		// Adding LOD 0 for merged mesh output
		DataTracker.AddLODIndex(0);

		// Retrieve mesh and section data for each component
		for (int32 ComponentIndex = 0; ComponentIndex < StaticMeshComponentsToMerge.Num(); ++ComponentIndex)
		{
			// Create material merge adapter for this component
			UStaticMeshComponent* Component = StaticMeshComponentsToMerge[ComponentIndex];
			Adapters.Add(FStaticMeshComponentAdapter(Component));
			FStaticMeshComponentAdapter& Adapter = Adapters.Last();

			// Determine LOD to use for merging, either user specified or calculated index and ensure we clamp to the maximum LOD index for this adapter 
			const int32 LODIndex = [&]()
			{
				int32 LowestDetailLOD = Adapter.GetNumberOfLODs() - 1;
				if (Component->HLODBatchingPolicy != EHLODBatchingPolicy::None && !InSettings.bIncludeImposters)
				{
					LowestDetailLOD = FMath::Max(0, LowestDetailLOD - 1);
				}

				switch (InSettings.LODSelectionType)
				{
				case EMeshLODSelectionType::SpecificLOD:
					return FMath::Min(LowestDetailLOD, InSettings.SpecificLOD);

				case EMeshLODSelectionType::CalculateLOD:
					return FMath::Min(LowestDetailLOD, Utilities->GetLODLevelForScreenSize(Component, FMath::Clamp(ScreenSize, 0.0f, 1.0f)));

				case EMeshLODSelectionType::LowestDetailLOD:
				default:
					return LowestDetailLOD;
				}
			}();

			RetrieveRawMeshData(DataTracker
				, ComponentIndex
				, LODIndex
				, Component
				, bPropagateMeshData
				, Sections
				, Adapter
				, bMergeMaterialData
				, InSettings);
		}
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ProcessRawMeshes);
		DataTracker.ProcessRawMeshes();
	}

	// Merge sockets
	TMap<FName, UStaticMeshSocket*> MergedSockets;
	if (InSettings.bMergeMeshSockets)
	{
		const FTransform PivotTransform = FTransform(MergedAssetPivot);
		for (UPrimitiveComponent* PrimitiveComponent : ComponentsToMerge)
		{
			if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(PrimitiveComponent))
			{
				if (UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh())
				{
					for (UStaticMeshSocket* Socket : StaticMesh->Sockets)
					{
						if (Socket)
						{
							UStaticMeshSocket* SocketCopy = DuplicateObject<UStaticMeshSocket>(Socket, nullptr);

						    // Fix name - rename if duplicates are found
							FString PlainName = SocketCopy->SocketName.GetPlainNameString();
						    int32 CurrentNumber = SocketCopy->SocketName.GetNumber();
						    while (MergedSockets.Contains(SocketCopy->SocketName))
						    {
							    SocketCopy->SocketName = FName(PlainName, CurrentNumber++);
						    }
    
						    // Fix transform - make relative to pivot
						    FTransform SocketTransformWorldSpace = StaticMeshComponent->GetSocketTransform(Socket->SocketName, RTS_World);
						    FTransform SocketTransformPivotSpace = SocketTransformWorldSpace.GetRelativeTransform(PivotTransform);
						    SocketCopy->RelativeLocation = SocketTransformPivotSpace.GetLocation();
						    SocketCopy->RelativeRotation = FRotator(SocketTransformPivotSpace.GetRotation());
						    SocketCopy->RelativeScale = SocketTransformPivotSpace.GetScale3D();

							MergedSockets.Add(SocketCopy->SocketName, SocketCopy);
						}
					}
				}
			}
		}
	}

	// Find all unique materials and remap section to unique materials
	TArray<UMaterialInterface*> UniqueMaterials;
	TMap<UMaterialInterface*, UMaterialInterface*> CollapsedMaterialMap;

	for (int32 SectionIndex = 0; SectionIndex < DataTracker.NumberOfUniqueSections(); ++SectionIndex)
	{
		// Unique index for material
		UMaterialInterface* MaterialInterface = DataTracker.GetMaterialForSectionIndex(SectionIndex);
		int32 UniqueIndex = UniqueMaterials.IndexOfByPredicate([&InSettings, MaterialInterface](const UMaterialInterface* InMaterialInterface)
			{
				// Perform an optional custom comparison if we are trying to collapse material instances
				if (InSettings.bMergeEquivalentMaterials)
				{
					return FMaterialKey(MaterialInterface) == FMaterialKey(InMaterialInterface);
				}
				return MaterialInterface == InMaterialInterface;
			});

		if (UniqueIndex == INDEX_NONE)
		{
			UniqueIndex = UniqueMaterials.Add(MaterialInterface);
		}

		// Update map to 'collapsed' materials
		CollapsedMaterialMap.Add(MaterialInterface, UniqueMaterials[UniqueIndex]);
	}

	// Retrieve physics data
	UBodySetup* BodySetupSource = nullptr;
	TArray<FKAggregateGeom> PhysicsGeometry;
	if (InSettings.bMergePhysicsData)
	{
		RetrievePhysicsData(ComponentsToMerge, PhysicsGeometry, BodySetupSource);
	}

	TMultiMap< FMeshLODKey, MaterialRemapPair > OutputMaterialsMap;

	// If the user wants to merge materials into a single one
	if (bMergeMaterialData && UniqueMaterials.Num() != 0)
	{
		// Create the merged material
		FFlattenMaterial FlattenMaterial;
		CreateMergedMaterial(DataTracker, InSettings, StaticMeshComponentsToMerge, Adapters, UniqueMaterials, CollapsedMaterialMap, OutputMaterialsMap, bMergeAllLODs, bMergeMaterialData, MergedAssetPivot, FlattenMaterial);
		if (FlattenMaterial.HasData())
		{
			// Don't recreate render states with the material update context as we will manually do it through
			// the FStaticMeshComponentRecreateRenderStateContext used below at the creation of the static mesh.
			FMaterialUpdateContext MaterialUpdateContext(FMaterialUpdateContext::EOptions::Default & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);

			UMaterialInterface* MergedMaterial = CreateProxyMaterial(InBasePackageName, MergedAssetPackageName, InBaseMaterial, InOuter, InSettings, FlattenMaterial, OutAssetsToSync, &MaterialUpdateContext);

			UniqueMaterials.Empty(1);
			UniqueMaterials.Add(MergedMaterial);

			FSectionInfo NewSection;
			NewSection.Material = MergedMaterial;
			NewSection.EnabledProperties.Add(GET_MEMBER_NAME_CHECKED(FStaticMeshSection, bCastShadow));
			DataTracker.AddBakedMaterialSection(NewSection);

			for (IMeshMergeExtension* Extension : MeshMergeExtensions)
			{
				Extension->OnCreatedProxyMaterial(StaticMeshComponentsToMerge, MergedMaterial);
			}
		}
	}

	// Create the merged mesh
	TArray<FMeshDescription> MergedRawMeshes;
	CreateMergedRawMeshes(DataTracker, InSettings, StaticMeshComponentsToMerge, UniqueMaterials, CollapsedMaterialMap, OutputMaterialsMap, bMergeAllLODs, bMergeMaterialData, MergedAssetPivot, MergedRawMeshes);

	// Notify listeners that our merged mesh was created
	for (IMeshMergeExtension* Extension : MeshMergeExtensions)
	{
		Extension->OnCreatedMergedRawMeshes(StaticMeshComponentsToMerge, DataTracker, MergedRawMeshes);
	}

	// Populate mesh section map
	FMeshSectionInfoMap SectionInfoMap;	
	for (TConstLODIndexIterator Iterator = DataTracker.GetLODIndexIterator(); Iterator; ++Iterator)
	{
		const int32 LODIndex = *Iterator;
		TArray<uint32> UniqueMaterialIndices;
		const FMeshDescription& TargetRawMesh = MergedRawMeshes[LODIndex];
		uint32 MaterialIndex = 0;
		for (FPolygonGroupID PolygonGroupID : TargetRawMesh.PolygonGroups().GetElementIDs())
		{
			//Skip empty group
			if (TargetRawMesh.GetPolygonGroupPolygonIDs(PolygonGroupID).Num() > 0)
			{
				if (PolygonGroupID.GetValue() < DataTracker.NumberOfUniqueSections())
				{
					UniqueMaterialIndices.AddUnique(PolygonGroupID.GetValue());
				}
				else
				{
					UniqueMaterialIndices.AddUnique(MaterialIndex);
				}
				MaterialIndex++;
			}
		}
		UniqueMaterialIndices.Sort();
		for (int32 Index = 0; Index < UniqueMaterialIndices.Num(); ++Index)
		{
			const int32 SectionIndex = UniqueMaterialIndices[Index];
			// unclear when this would not be the case, but it seems to be able to occur
			if (SectionIndex < DataTracker.NumberOfUniqueSections())
			{
				const FSectionInfo& StoredSectionInfo = DataTracker.GetSection(SectionIndex);
				FMeshSectionInfo SectionInfo;
				SectionInfo.bCastShadow = StoredSectionInfo.EnabledProperties.Contains(GET_MEMBER_NAME_CHECKED(FMeshSectionInfo, bCastShadow));
				SectionInfo.bEnableCollision = StoredSectionInfo.EnabledProperties.Contains(GET_MEMBER_NAME_CHECKED(FMeshSectionInfo, bEnableCollision));
				SectionInfo.MaterialIndex = UniqueMaterials.Num() == 1 ? 0 : UniqueMaterials.IndexOfByKey(CollapsedMaterialMap[StoredSectionInfo.Material]);
				SectionInfoMap.Set(LODIndex, Index, SectionInfo);
			}
		}
	}

	// Transform physics primitives to merged mesh pivot
	if (InSettings.bMergePhysicsData && !MergedAssetPivot.IsZero())
	{
		FTransform PivotTM(-MergedAssetPivot);
		for (FKAggregateGeom& Geometry : PhysicsGeometry)
		{
			FMeshMergeHelpers::TransformPhysicsGeometry(PivotTM, false, Geometry);
		}
	}

	// Compute target lightmap channel for each LOD, by looking at the first empty UV channel
	const int32 LightMapUVChannel = [&]()
	{
		if (InSettings.bGenerateLightMapUV)
		{
			const int32 TempChannel = DataTracker.GetAvailableLightMapUVChannel();
			if (TempChannel == INDEX_NONE)
			{
				// Output warning message
				UE_LOGF(LogMeshMerging, Warning, "Failed to find an available channel for Lightmap UVs. Lightmap UVs will not be generated.");
			}
			return TempChannel;
		}

		return (int32)INDEX_NONE;
	}();

	//
	//Create merged mesh asset
	//

	MergedRawMeshes.SetNum(Algo::RemoveIf(MergedRawMeshes, [](const FMeshDescription& MeshDescription) { return MeshDescription.IsEmpty(); }));

	const bool bContainsImposters = !ImposterComponents.IsEmpty();
	const bool bContainsMergedMeshes = !MergedRawMeshes.IsEmpty();

	if (bContainsMergedMeshes || bContainsImposters)
	{
		FString AssetName;
		FString PackageName;
		if (InBasePackageName.IsEmpty())
		{
			AssetName = TEXT("SM_MERGED_") + FPackageName::GetShortName(MergedAssetPackageName);
			PackageName = FPackageName::GetLongPackagePath(MergedAssetPackageName) / AssetName;
		}
		else
		{
			AssetName = TEXT("SM_") + FPackageName::GetShortName(InBasePackageName);
			PackageName = FPackageName::GetLongPackagePath(InBasePackageName) / AssetName;
		}

		UPackage* Package = InOuter;
		if (Package == nullptr)
		{
			Package = CreatePackage( *PackageName);
			check(Package);
			Package->FullyLoad();
			Package->Modify();
		}

		// Check that an asset of a different class does not already exist
		{
			UObject* ExistingObject = StaticFindObject( nullptr, Package, *AssetName);
			if(ExistingObject && !ExistingObject->GetClass()->IsChildOf(UStaticMesh::StaticClass()))
			{
				// Change name of merged static mesh to avoid name collision
				UPackage* ParentPackage = CreatePackage( *FPaths::GetPath(Package->GetPathName()));
				ParentPackage->FullyLoad();

				AssetName = MakeUniqueObjectName( ParentPackage, UStaticMesh::StaticClass(), *AssetName).ToString();
				Package = CreatePackage( *(ParentPackage->GetPathName() / AssetName ));
				check(Package);
				Package->FullyLoad();
				Package->Modify();

				// Let user know name of merged static mesh has changed
				UE_LOGF(LogMeshMerging, Warning,
					"Cannot create %ls %ls.%ls\n" "An object with the same fully qualified name but a different class already exists.\n" "\tExisting Object: %ls\n" "The merged mesh will be named %ls.%ls",
					*UStaticMesh::StaticClass()->GetName(), *ExistingObject->GetOutermost()->GetPathName(),	*ExistingObject->GetName(),
					*ExistingObject->GetFullName(), *Package->GetPathName(), *AssetName);
			}
		}

		FStaticMeshComponentRecreateRenderStateContext RecreateRenderStateContext(FindObject<UStaticMesh>(Package, *AssetName));

		UStaticMesh* StaticMesh = NewObject<UStaticMesh>(Package, *AssetName, RF_Public | RF_Standalone);
		StaticMesh->InitResources();

		FString OutputPath = StaticMesh->GetPathName();

		// make sure it has a new lighting guid
		StaticMesh->SetLightingGuid();
		if (LightMapUVChannel != INDEX_NONE)
		{
			StaticMesh->SetLightMapResolution(InSettings.TargetLightMapResolution);
			StaticMesh->SetLightMapCoordinateIndex(LightMapUVChannel);
		}

		// Ray tracing support
		StaticMesh->bSupportRayTracing = InSettings.bSupportRayTracing;

		TArray<UMaterialInterface*> ImposterMaterials;
		FBox ImposterBounds(EForceInit::ForceInit);
		for (int32 LODIndex = 0; LODIndex < MergedRawMeshes.Num(); ++LODIndex)
		{
			FMeshDescription& MergedMeshLOD = MergedRawMeshes[LODIndex];
			if (MergedMeshLOD.Vertices().Num() > 0 || bContainsImposters)
			{
				FStaticMeshSourceModel& SrcModel = StaticMesh->AddSourceModel();

				// Don't allow the engine to recalculate normals
				SrcModel.BuildSettings.bRecomputeNormals = false;
				SrcModel.BuildSettings.bRecomputeTangents = false;
				SrcModel.BuildSettings.bRemoveDegenerates = false;
				SrcModel.BuildSettings.bUseHighPrecisionTangentBasis = false;
				SrcModel.BuildSettings.bUseFullPrecisionUVs = false;
				SrcModel.BuildSettings.bBuildReversedIndexBuffer = false;
				SrcModel.BuildSettings.bGenerateLightmapUVs = LightMapUVChannel != INDEX_NONE;
				SrcModel.BuildSettings.MinLightmapResolution = InSettings.bComputedLightMapResolution ? DataTracker.GetLightMapDimension() : InSettings.TargetLightMapResolution;
				SrcModel.BuildSettings.SrcLightmapIndex = 0;
				SrcModel.BuildSettings.DstLightmapIndex = LightMapUVChannel != INDEX_NONE ? LightMapUVChannel : 0;
				if(!InSettings.bAllowDistanceField)
				{
					SrcModel.BuildSettings.DistanceFieldResolutionScale = 0.0f;
				}

				if (bContainsImposters)
				{
					// Merge imposter meshes to rawmesh
					FMeshMergeHelpers::MergeImpostersToMesh(ImposterComponents, MergedMeshLOD, MergedAssetPivot, UniqueMaterials.Num(), ImposterMaterials);

					const FTransform PivotTransform = FTransform(MergedAssetPivot);
					for (const UStaticMeshComponent* Component : ImposterComponents)
					{
						if (Component->GetStaticMesh())
						{
							ImposterBounds += Component->GetStaticMesh()->GetBoundingBox().TransformBy(Component->GetComponentToWorld().GetRelativeTransform(PivotTransform));
						}
					}
				}

				FMeshDescription* MeshDescription = StaticMesh->CreateMeshDescription(LODIndex, MergedMeshLOD);

				UStaticMesh::FCommitMeshDescriptionParams CommitParams;
				CommitParams.bUseHashAsGuid = true;
				StaticMesh->CommitMeshDescription(LODIndex, CommitParams);
			}
		}
		
		auto IsMaterialImportedNameUnique = [&StaticMesh](FName ImportedMaterialSlotName)
		{
			for (const FStaticMaterial& StaticMaterial : StaticMesh->GetStaticMaterials())
			{
#if WITH_EDITOR
				if (StaticMaterial.ImportedMaterialSlotName == ImportedMaterialSlotName)
#else
				if (StaticMaterial.MaterialSlotName == ImportedMaterialSlotName)
#endif
				{
					return false;
				}
			}
			return true;
		};
		

		for (UMaterialInterface* Material : UniqueMaterials)
		{
			if (Material && (!Material->IsAsset() && InOuter != GetTransientPackage()))
			{
				// Only allow duplication for MIDs and MICs
				if (Material->IsA<UMaterialInstanceDynamic>() || Material->IsA<UMaterialInstanceConstant>())
				{
					// Duplicate them and outer them to the static mesh.
					Material = DuplicateObject<UMaterialInterface>(Material, StaticMesh);
				}
				else
				{
					UE_LOGF(LogMeshMerging, Error, "Merged mesh is referencing a non-asset material (\"%ls\"). This material can't be referenced outside it's package as it is not public. Clearing out material entry in the final mesh.", *Material->GetPathName());
					Material = nullptr; // do not save non-asset materials
				}
			}

			//Make sure we have unique slot name here
			FName MaterialSlotName = DataTracker.GetMaterialSlotName(Material);
			int32 Counter = 1;
			while (!IsMaterialImportedNameUnique(MaterialSlotName))
			{
				MaterialSlotName = *(DataTracker.GetMaterialSlotName(Material).ToString() + TEXT("_") + FString::FromInt(Counter++));
			}

			StaticMesh->GetStaticMaterials().Add(FStaticMaterial(Material, MaterialSlotName));
		}

		for(UMaterialInterface* ImposterMaterial : ImposterMaterials)
		{
			//Make sure we have unique slot name here
			FName MaterialSlotName = ImposterMaterial->GetFName();
			int32 Counter = 1;
			while (!IsMaterialImportedNameUnique(MaterialSlotName))
			{
				MaterialSlotName = *(ImposterMaterial->GetName() + TEXT("_") + FString::FromInt(Counter++));
			}
			StaticMesh->GetStaticMaterials().Add(FStaticMaterial(ImposterMaterial, MaterialSlotName));
		}

		if (InSettings.bMergePhysicsData)
		{
			StaticMesh->CreateBodySetup();
			if (BodySetupSource)
			{
				StaticMesh->GetBodySetup()->CopyBodyPropertiesFrom(BodySetupSource);
			}

			StaticMesh->GetBodySetup()->AggGeom = FKAggregateGeom();
			// Copy collision from the source meshes
			for (const FKAggregateGeom& Geom : PhysicsGeometry)
			{
				StaticMesh->GetBodySetup()->AddCollisionFrom(Geom);
			}

			// Bake rotation into verts of convex hulls, so they scale correctly after rotation
			for (FKConvexElem& ConvexElem : StaticMesh->GetBodySetup()->AggGeom.ConvexElems)
			{
				ConvexElem.BakeTransformToVerts();
			}
		}

		// Add merged sockets
		if (InSettings.bMergeMeshSockets)
		{
			for (auto& [SocketName, Socket] : MergedSockets)
			{
				Socket->Rename(nullptr, StaticMesh);
				StaticMesh->AddSocket(Socket);
			}
		}

		StaticMesh->GetSectionInfoMap().CopyFrom(SectionInfoMap);
		StaticMesh->GetOriginalSectionInfoMap().CopyFrom(SectionInfoMap);

		//Set the Imported version before calling the build
		StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);
		StaticMesh->SetLightMapResolution(InSettings.bComputedLightMapResolution ? DataTracker.GetLightMapDimension() : InSettings.TargetLightMapResolution);

		// Nanite settings
		StaticMesh->SetNaniteSettings(InSettings.NaniteSettings);

#if WITH_EDITOR
		//If we are running the automation test
		if (GIsAutomationTesting)
		{
			StaticMesh->BuildCacheAutomationTestGuid = FGuid::NewGuid();
		}
#endif

		// Ensure the new mesh is not referencing non standalone materials
		FMeshMergeHelpers::FixupNonStandaloneMaterialReferences(StaticMesh);


		if (ImposterBounds.IsValid)
		{
			const FBox StaticMeshBox = StaticMesh->GetBoundingBox();
			const FBox CombinedBox = StaticMeshBox + ImposterBounds;
			StaticMesh->SetPositiveBoundsExtension((CombinedBox.Max - StaticMeshBox.Max));
			StaticMesh->SetNegativeBoundsExtension((StaticMeshBox.Min - CombinedBox.Min));
			StaticMesh->CalculateExtendedBounds();
		}		

		UStaticMesh::FBuildParameters BuildParameters;
		BuildParameters.bInSilent = true;
		BuildParameters.bInRebuildUVChannelData = true;
		BuildParameters.bInEnforceLightmapRestrictions = true;

		StaticMesh->Build(BuildParameters);

		OutAssetsToSync.Add(StaticMesh);
		OutMergedActorLocation = MergedAssetPivot;
	}
	else
	{
		UE_LOGF(LogMeshMerging, Display, "MergeComponentsToStaticMesh: Skipped creation of a static mesh asset as no input meshes were provided");
	}
}

void FMeshMergeUtilities::CreateMergedMaterial(FMeshMergeDataTracker& InDataTracker, const FMeshMergingSettings& InSettings, const TArray<UStaticMeshComponent*>& InStaticMeshComponentsToMerge, TArray<FStaticMeshComponentAdapter>& InAdapters, const TArray<UMaterialInterface*>& InUniqueMaterials, const TMap<UMaterialInterface*, UMaterialInterface*>& InCollapsedMaterialMap, TMultiMap<FMeshLODKey, MaterialRemapPair>& InOutputMaterialsMap, bool bInMergeAllLODs, bool bInMergeMaterialData, const FVector& InMergedAssetPivot, FFlattenMaterial& OutFlattenMaterial) const
{
	OutFlattenMaterial.ReleaseData();

	TArray<UPrimitiveComponent*> PrimitiveComponents;
	Algo::Transform(InStaticMeshComponentsToMerge, PrimitiveComponents, [](UStaticMeshComponent* SMComponent) { return SMComponent; });

	FMaterialProxySettings MaterialProxySettings = InSettings.MaterialSettings;
	if (MaterialProxySettings.ResolveTexelDensity(PrimitiveComponents))
	{
		MaterialProxySettings.TextureSize = InDataTracker.GetTextureSizeFromTargetTexelDensity(MaterialProxySettings.TargetTexelDensityPerMeter);
		MaterialProxySettings.TextureSizingType = ETextureSizingType::TextureSizingType_UseSingleTextureSize;
	}

	UMaterialOptions* MaterialOptions = PopulateMaterialOptions(MaterialProxySettings);
	TGCObjectScopeGuard<UMaterialOptions> MaterialOptionsGCScopeGuard(MaterialOptions);

	// Check each material to see if the shader actually uses vertex data / world position.
	// Deferred and cached per material, as AnalyzeMaterialPropertyEx() is a costly call.
	TArray<TOptional<FMaterialDataUsage>> CachedMaterialDataUsage;
	CachedMaterialDataUsage.SetNum(InUniqueMaterials.Num());

	auto GetMaterialDataUsage = [&](const int32 InMaterialIndex)
	{
		if (!CachedMaterialDataUsage[InMaterialIndex].IsSet())
		{
			CachedMaterialDataUsage[InMaterialIndex] = DetermineMaterialDataUsage(InUniqueMaterials[InMaterialIndex], MaterialOptions);
		}

		return CachedMaterialDataUsage[InMaterialIndex].GetValue();
	};

	auto DoesMaterialUsesVertexData = [&](const int32 InMaterialIndex)
	{
		return GetMaterialDataUsage(InMaterialIndex).bUsesVertexData;
	};

	auto DoesMaterialUseWorldPosition = [&](const int32 InMaterialIndex)
	{
		return GetMaterialDataUsage(InMaterialIndex).bUsesWorldPosition;
	};

	// For each unique material calculate how 'important' they are
	TArray<float> MaterialImportanceValues;
	FMaterialUtilities::DetermineMaterialImportance(InUniqueMaterials, MaterialImportanceValues);

	TArray<FMeshData> GlobalMeshSettings;
	TArray<FMaterialData> GlobalMaterialSettings;
	TArray<float> SectionMaterialImportanceValues;

	TMap<EMaterialProperty, FIntPoint> PropertySizes;
	for (const FPropertyEntry& Entry : MaterialOptions->Properties)
	{
		if (!Entry.bUseConstantValue && Entry.Property != MP_MAX)
		{
			PropertySizes.Add(Entry.Property, Entry.bUseCustomSize ? Entry.CustomSize : MaterialOptions->TextureSize);
		}
	}

	// If we are generating a single LOD and want to merge materials we can utilize texture space better by generating unique UVs
	// for the merged mesh and baking out materials using those UVs
	const bool bGloballyRemapUVs = !bInMergeAllLODs && !InSettings.bReuseMeshLightmapUVs;

	typedef TTuple<UStaticMesh*, int32> FMeshLODTuple;
	typedef TFuture<TArray<FVector2D>> FUVComputeFuture;
	TMap<FMeshLODTuple, FUVComputeFuture> MeshLODsTextureCoordinates;
	TMap<int32, FMeshLODTuple> MeshDataAwaitingResults;

	for (TConstRawMeshIterator RawMeshIterator = InDataTracker.GetConstRawMeshIterator(); RawMeshIterator; ++RawMeshIterator)
	{
		const FMeshLODKey& Key = RawMeshIterator.Key();
		const FMeshDescription& RawMesh = RawMeshIterator.Value();
		const bool bRequiresUniqueUVs = InDataTracker.DoesMeshLODRequireUniqueUVs(Key);

		const FMeshDescription* MeshDescription = InDataTracker.GetRawMeshPtr(Key);
		UStaticMeshComponent* Component = InStaticMeshComponentsToMerge[Key.GetMeshIndex()];
		UStaticMesh* StaticMesh = Component->GetStaticMesh();

		// Retrieve all sections and materials for key
		TArray<SectionRemapPair> SectionRemapPairs;
		InDataTracker.GetMappingsForMeshLOD(Key, SectionRemapPairs);

		// Contains unique materials used for this key, and the accompanying section index which point to the material
		TMap<UMaterialInterface*, TArray<int32>> MaterialAndSectionIndices;

		for (const SectionRemapPair& RemapPair : SectionRemapPairs)
		{
			const int32 UniqueIndex = RemapPair.Value;
			const int32 SectionIndex = RemapPair.Key;
			TArray<int32>& SectionIndices = MaterialAndSectionIndices.FindOrAdd(InCollapsedMaterialMap.FindChecked(InDataTracker.GetMaterialForSectionIndex(UniqueIndex)));
			SectionIndices.Add(SectionIndex);
		}

		for (TPair<UMaterialInterface*, TArray<int32>>& MaterialSectionIndexPair : MaterialAndSectionIndices)
		{
			UMaterialInterface* Material = MaterialSectionIndexPair.Key;
			const int32 MaterialIndex = InUniqueMaterials.IndexOfByKey(Material);
			const TArray<int32>& SectionIndices = MaterialSectionIndexPair.Value;

			FMaterialData MaterialData;
			MaterialData.Material = InCollapsedMaterialMap.FindChecked(Material);
			MaterialData.PropertySizes = PropertySizes;

			FMeshData NewMeshData;
			const bool bUseMeshData = bGloballyRemapUVs || (InSettings.bUseVertexDataForBakingMaterial && (bRequiresUniqueUVs || DoesMaterialUsesVertexData(MaterialIndex)));
			if (bUseMeshData)
			{
				NewMeshData.Mesh = Key.GetMesh();
				NewMeshData.MeshDescription = MeshDescription;
				NewMeshData.VertexColorHash = Key.GetVertexColorHash();
				NewMeshData.bMirrored = Component->GetComponentTransform().GetDeterminant() < 0.0f;
				NewMeshData.MaterialIndices = SectionIndices;
				if (!Component->GetCustomPrimitiveData().Data.IsEmpty())
				{
					NewMeshData.PrimitiveData = FPrimitiveData();
					NewMeshData.PrimitiveData->CustomPrimitiveData = &Component->GetCustomPrimitiveData();
				}
			}

			auto CompareMaterialData = [&InSettings](const FMaterialData& LHS, const FMaterialData& RHS)
			{
				return InSettings.bMergeEquivalentMaterials ? FMaterialKey(LHS.Material) == FMaterialKey(RHS.Material) : LHS.Material == RHS.Material;
			};

			auto CompareCustomPrimitiveData = [](const FCustomPrimitiveData* LHS, const FCustomPrimitiveData* RHS)
			{
				// Return true if both are null, false if one of them is null - otherwise, compare content
				return (!LHS && !RHS) ? true : (!LHS || !RHS) ? false : (*LHS == *RHS);
			};

			auto ComparePrimitiveData = [&CompareCustomPrimitiveData](const TOptional<FPrimitiveData>& LHS, const TOptional<FPrimitiveData>& RHS)
			{
				// Return true if both are null, false if one of them is null - otherwise, compare content
				return (!LHS && !RHS) ? true : (!LHS || !RHS) ? false : CompareCustomPrimitiveData(LHS->CustomPrimitiveData, RHS->CustomPrimitiveData);
			};

			// When the material reads world position, instances of the same mesh at different world positions must not share a bake entry, as they produce different results.
			const bool bMaterialNeedsUniqueInstances = DoesMaterialUseWorldPosition(MaterialIndex);

			auto CompareMeshData = [&ComparePrimitiveData, bMaterialNeedsUniqueInstances](const FMeshData& LHS, const FMeshData& RHS)
			{
				return (LHS.Mesh == RHS.Mesh) && (!bMaterialNeedsUniqueInstances || LHS.MeshDescription == RHS.MeshDescription) && (LHS.MaterialIndices == RHS.MaterialIndices) && (LHS.bMirrored == RHS.bMirrored) && (LHS.VertexColorHash == RHS.VertexColorHash) && ComparePrimitiveData(LHS.PrimitiveData, RHS.PrimitiveData);
			};

			// Find material & mesh pair
			int32 MeshDataIndex = INDEX_NONE;
			for (int32 GlobalMaterialSettingsIndex = 0; GlobalMaterialSettingsIndex < GlobalMaterialSettings.Num(); ++GlobalMaterialSettingsIndex)
			{
				if (CompareMaterialData(GlobalMaterialSettings[GlobalMaterialSettingsIndex], MaterialData) &&
					CompareMeshData(GlobalMeshSettings[GlobalMaterialSettingsIndex], NewMeshData))
				{
					MeshDataIndex = GlobalMaterialSettingsIndex;
					break;
				}
			}

			// We've found a match, no need to process this mesh/material pair
			if (MeshDataIndex == INDEX_NONE)
			{
				// We're processing a new pair
				MeshDataIndex = GlobalMeshSettings.Num();

				FMeshData& MeshData = GlobalMeshSettings.Emplace_GetRef(NewMeshData);
				GlobalMaterialSettings.Add(MaterialData);
				SectionMaterialImportanceValues.Add(MaterialImportanceValues[MaterialIndex]);

				if (bUseMeshData)
				{
					// if it has vertex color/*WedgetColors.Num()*/, it should also use light map UV index
					// we can't do this for all meshes, but only for the mesh that has vertex color.
					if (bRequiresUniqueUVs || MeshData.MeshDescription->VertexInstances().Num() > 0)
					{
						// Check if there are lightmap uvs available?
						const int32 LightMapUVIndex = StaticMesh->GetLightMapCoordinateIndex();

						TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = FStaticMeshConstAttributes(*MeshData.MeshDescription).GetVertexInstanceUVs();
						if (InSettings.bReuseMeshLightmapUVs && VertexInstanceUVs.GetNumElements() > 0 && VertexInstanceUVs.GetNumChannels() > LightMapUVIndex)
						{
							MeshData.TextureCoordinateIndex = LightMapUVIndex;
						}
						else
						{
							// Verify if we started an async task to generate UVs for this static mesh & LOD
							FMeshLODTuple Tuple(Key.GetMesh(), Key.GetLODIndex());
							if (!MeshLODsTextureCoordinates.Find(Tuple))
							{
								// No job found yet, fire an async task
								MeshLODsTextureCoordinates.Add(Tuple, Async(EAsyncExecution::Thread, [MeshDescription, MaterialOptions, this]()
								{
									FStaticMeshOperations::FGenerateUVOptions GenerateUVOptions;
									GenerateUVOptions.TextureResolution = MaterialOptions->TextureSize.GetMax();
									GenerateUVOptions.bMergeTrianglesWithIdenticalAttributes = false;
									GenerateUVOptions.UVMethod = GetUVGenerationMethodToUse();

									TArray<FVector2D> UniqueTextureCoordinates;
									FStaticMeshOperations::GenerateUV(*MeshDescription, GenerateUVOptions, UniqueTextureCoordinates);

									if (GenerateUVOptions.UVMethod == FStaticMeshOperations::EGenerateUVMethod::Legacy)
									{
										ScaleTextureCoordinatesToBox(FBox2D(FVector2D::ZeroVector, FVector2D(1, 1)), UniqueTextureCoordinates);
									}

									return UniqueTextureCoordinates;
								}));
							}
							// Keep track of the fact that this mesh is waiting for the UV computation to finish
							MeshDataAwaitingResults.Add(MeshDataIndex, Tuple);
						}
					}

					InAdapters[Key.GetMeshIndex()].ApplySettings(Key.GetLODIndex(), MeshData);
				}
			}

			for (const auto& OriginalSectionIndex : SectionIndices)
			{
				InOutputMaterialsMap.Add(Key, MaterialRemapPair(OriginalSectionIndex, MeshDataIndex));
			}
		}
	}

	// Fetch results from the async UV computation tasks
	for (auto MeshData : MeshDataAwaitingResults)
	{
		GlobalMeshSettings[MeshData.Key].CustomTextureCoordinates = MeshLODsTextureCoordinates[MeshData.Value].Get();
	}

	TArray<FMeshData*> MeshSettingPtrs;
	for (int32 SettingsIndex = 0; SettingsIndex < GlobalMeshSettings.Num(); ++SettingsIndex)
	{
		MeshSettingPtrs.Add(&GlobalMeshSettings[SettingsIndex]);
	}

	TArray<FMaterialData*> MaterialSettingPtrs;
	for (int32 SettingsIndex = 0; SettingsIndex < GlobalMaterialSettings.Num(); ++SettingsIndex)
	{
		MaterialSettingPtrs.Add(&GlobalMaterialSettings[SettingsIndex]);
	}

	if (bGloballyRemapUVs)
	{
		// Adjust merge settings when merging for the unique UV/material baking pass
		// The final merged mesh will use the original settings
		FMeshMergingSettings RemapUVMergeSettings = InSettings;
		
		// Keep vertex data in order to properly generate unique UVs
		RemapUVMergeSettings.bBakeVertexDataToMesh = true;

		// Keep all UVs as some channels might be needed to properly bake the material
		for (EUVOutput& OutputUV : RemapUVMergeSettings.OutputUVs)
		{
			OutputUV = EUVOutput::OutputChannel;
		}
		
		TArray<FMeshDescription> MergedRawMeshes;
		CreateMergedRawMeshes(InDataTracker, RemapUVMergeSettings, InStaticMeshComponentsToMerge, InUniqueMaterials, InCollapsedMaterialMap, InOutputMaterialsMap, false, false, InMergedAssetPivot, MergedRawMeshes);

		// Create texture coords for the merged mesh
		// Only merge identical triangles if no material reads world position,
		// as instances of the same mesh at different world positions would otherwise share
		// UV space in the atlas and produce identical baked results.
		bool bAnyMaterialUsesWorldPosition = false;
		for (int32 MaterialIndex = 0; MaterialIndex < InUniqueMaterials.Num(); ++MaterialIndex)
		{
			if (DoesMaterialUseWorldPosition(MaterialIndex))
			{
				bAnyMaterialUsesWorldPosition = true;
				break;
			}
		}

		FStaticMeshOperations::FGenerateUVOptions GenerateUVOptions;
		GenerateUVOptions.TextureResolution = MaterialOptions->TextureSize.GetMax();
		GenerateUVOptions.bMergeTrianglesWithIdenticalAttributes = !bAnyMaterialUsesWorldPosition;
		GenerateUVOptions.UVMethod = GetUVGenerationMethodToUse();

		TArray<FVector2D> GlobalTextureCoordinates;
		bool bSuccess = FStaticMeshOperations::GenerateUV(MergedRawMeshes[0], GenerateUVOptions, GlobalTextureCoordinates);
		if (bSuccess)
		{
			if (GenerateUVOptions.UVMethod == FStaticMeshOperations::EGenerateUVMethod::Legacy)
			{
				ScaleTextureCoordinatesToBox(FBox2D(FVector2D::ZeroVector, FVector2D(1, 1)), GlobalTextureCoordinates);
			}

			// copy UVs back to the un-merged mesh's custom texture coords
			// iterate the raw meshes in the same way as when we combined the mesh above in CreateMergedRawMeshes()
			int32 GlobalUVIndex = 0;
			for (TConstRawMeshIterator RawMeshIterator = InDataTracker.GetConstRawMeshIterator(); RawMeshIterator; ++RawMeshIterator)
			{
				const FMeshLODKey& Key = RawMeshIterator.Key();
				const FMeshDescription& RawMesh = RawMeshIterator.Value();

				// Build a local array for this raw mesh
				TArray<FVector2D> UniqueTextureCoordinates;
				UniqueTextureCoordinates.SetNumUninitialized(RawMesh.VertexInstances().Num());
				for (FVector2D& UniqueTextureCoordinate : UniqueTextureCoordinates)
				{
					UniqueTextureCoordinate = GlobalTextureCoordinates[GlobalUVIndex++];
				}

				// copy to mesh data
				for (FMeshData& MeshData : GlobalMeshSettings)
				{
					if (MeshData.MeshDescription == &RawMesh)
					{
						MeshData.CustomTextureCoordinates = UniqueTextureCoordinates;
					}
				}
			}

			// Dont smear borders as we will copy back non-pink pixels
			for (FMaterialData& MaterialData : GlobalMaterialSettings)
			{
				MaterialData.bPerformBorderSmear = false;
			}
		}
		else
		{
			UE_LOGF(LogMeshMerging, Warning, "GenerateUV: Failed to pack UVs for static mesh");
		}
	}

	TArray<FFlattenMaterial> FlattenedMaterials;
	// This scope ensures BakeOutputs is never used after TransferOutputToFlatMaterials
	{
		TArray<FBakeOutput> BakeOutputs;
		IMaterialBakingModule& Module = FModuleManager::Get().LoadModuleChecked<IMaterialBakingModule>("MaterialBaking");

		// If we're working with a new set of UVs, we can bake all materials directly to the same bake output
		// as our remapped UVs for each mesh don't overlap.
		if (bGloballyRemapUVs)
		{
			FBakeOutput& BakeOutput = BakeOutputs.Emplace_GetRef();
			Module.BakeMaterials(MaterialSettingPtrs, MeshSettingPtrs, BakeOutput);
		}
		else
		{
			Module.BakeMaterials(MaterialSettingPtrs, MeshSettingPtrs, BakeOutputs);
		}

		// Append constant properties ?
		TArray<FColor> ConstantData;
		FIntPoint ConstantSize(1, 1);
		for (const FPropertyEntry& Entry : MaterialOptions->Properties)
		{
			if (Entry.bUseConstantValue && Entry.Property != MP_MAX)
			{
				ConstantData.SetNum(1, EAllowShrinking::No);
				ConstantData[0] = FLinearColor(Entry.ConstantValue, Entry.ConstantValue, Entry.ConstantValue).ToFColor(true);
				for (FBakeOutput& Output : BakeOutputs)
				{
					Output.PropertyData.Add(Entry.Property, ConstantData);
					Output.PropertySizes.Add(Entry.Property, ConstantSize);
				}
			}
		}

		TransferOutputToFlatMaterials(GlobalMaterialSettings, BakeOutputs, FlattenedMaterials);
	}

	if (!bGloballyRemapUVs)
	{
		// Try to optimize materials where possible	
		for (FFlattenMaterial& InMaterial : FlattenedMaterials)
		{
			FMaterialUtilities::OptimizeFlattenMaterial(InMaterial);
		}
	}

	for (const FPropertyEntry& Entry : MaterialOptions->Properties)
	{
		if (Entry.Property != MP_MAX)
		{
			EFlattenMaterialProperties OldProperty = ToFlattenProperty(Entry.Property);
			if (ensure(OldProperty != EFlattenMaterialProperties::NumFlattenMaterialProperties))
			{
				OutFlattenMaterial.SetPropertySize(OldProperty, Entry.bUseCustomSize ? Entry.CustomSize : MaterialOptions->TextureSize);
			}
		}
	}

	TArray<FUVOffsetScalePair> UVTransforms;
	if (bGloballyRemapUVs)
	{
		// If we have globally remapped UVs we copy non-pink pixels over the dest texture rather than 
		// copying sub-charts
		TArray<FBox2D> MaterialBoxes;
		MaterialBoxes.SetNumUninitialized(GlobalMaterialSettings.Num());
		for (FBox2D& Box2D : MaterialBoxes)
		{
			Box2D = FBox2D(FVector2D(0.0f, 0.0f), FVector2D(1.0f, 1.0f));
		}

		FlattenBinnedMaterials(FlattenedMaterials, MaterialBoxes, 0, true, OutFlattenMaterial, UVTransforms);

		static const FUVOffsetScalePair NoUVTransform = { FVector2D::Zero(), FVector2D::One() };
		UVTransforms.Init(NoUVTransform, GlobalMaterialSettings.Num());
	}
	else
	{
		/** Reweighting */
		float TotalValue = 0.0f;
		for (const float& Value : SectionMaterialImportanceValues)
		{
			TotalValue += Value;
		}

		float Multiplier = 1.0f / TotalValue;

		for (float& Value : SectionMaterialImportanceValues)
		{
			Value *= Multiplier;
		}
		/** End reweighting */

		if (InSettings.bUseTextureBinning)
		{
			TArray<FBox2D> MaterialBoxes;
			FMaterialUtilities::GeneratedBinnedTextureSquares(FVector2D(1.0f, 1.0f), SectionMaterialImportanceValues, MaterialBoxes);
			FlattenBinnedMaterials(FlattenedMaterials, MaterialBoxes, InSettings.GutterSize, false, OutFlattenMaterial, UVTransforms);
		}
		else
		{
			MergeFlattenedMaterials(FlattenedMaterials, InSettings.GutterSize, OutFlattenMaterial, UVTransforms);
		}
	}

	// Adjust UVs
	for (int32 ComponentIndex = 0; ComponentIndex < InStaticMeshComponentsToMerge.Num(); ++ComponentIndex)
	{
		TArray<uint32> ProcessedMaterials;
		for (TPair<FMeshLODKey, MaterialRemapPair>& MappingPair : InOutputMaterialsMap)
		{
			if (MappingPair.Key.GetMeshIndex() == ComponentIndex && !ProcessedMaterials.Contains(MappingPair.Value.Key))
			{
				// Retrieve raw mesh data for this component and lod pair
				FMeshDescription* RawMesh = InDataTracker.GetRawMeshPtr(MappingPair.Key);

				FMeshData& MeshData = GlobalMeshSettings[MappingPair.Value.Value];
				const FUVOffsetScalePair& UVTransform = UVTransforms[MappingPair.Value.Value];

				const uint32 MaterialIndex = MappingPair.Value.Key;
				ProcessedMaterials.Add(MaterialIndex);
				if (RawMesh->Vertices().Num())
				{
					TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = FStaticMeshAttributes(*RawMesh).GetVertexInstanceUVs();
					int32 NumUVChannel = FMath::Min(VertexInstanceUVs.GetNumChannels(), (int32)MAX_MESH_TEXTURE_COORDS);
					for (int32 UVChannelIdx = 0; UVChannelIdx < NumUVChannel; ++UVChannelIdx)
					{
						int32 VertexIndex = 0;
						for (FVertexInstanceID VertexInstanceID : RawMesh->VertexInstances().GetElementIDs())
						{
							FVector2D UV = FVector2D(VertexInstanceUVs.Get(VertexInstanceID, UVChannelIdx));
							if (UVChannelIdx == 0)
							{
								if (MeshData.CustomTextureCoordinates.Num())
								{
									UV = MeshData.CustomTextureCoordinates[VertexIndex];
								}
								else if (MeshData.TextureCoordinateIndex != 0)
								{
									check(MeshData.TextureCoordinateIndex < NumUVChannel);
									UV = FVector2D(VertexInstanceUVs.Get(VertexInstanceID, MeshData.TextureCoordinateIndex));
								}
							}

							const TArray<FPolygonID>& Polygons = RawMesh->GetVertexInstanceConnectedPolygons(VertexInstanceID);
							for (FPolygonID PolygonID : Polygons)
							{
								FPolygonGroupID PolygonGroupID = RawMesh->GetPolygonPolygonGroup(PolygonID);
								if (PolygonGroupID.GetValue() == MaterialIndex)
								{
									if (UVTransform.Value != FVector2D::ZeroVector)
									{
										VertexInstanceUVs.Set(VertexInstanceID, UVChannelIdx, FVector2f(UV * UVTransform.Value + UVTransform.Key));	// LWC_TODO: Precision loss
										break;
									}
								}
							}
							VertexIndex++;
						}
					}
				}
			}
		}
	}

	for (TRawMeshIterator Iterator = InDataTracker.GetRawMeshIterator(); Iterator; ++Iterator)
	{
		FMeshDescription& RawMesh = Iterator.Value();
		// Reset material indexes
		TMap<FPolygonGroupID, FPolygonGroupID> RemapPolygonGroups;
		for (FPolygonGroupID PolygonGroupID : RawMesh.PolygonGroups().GetElementIDs())
		{
			RemapPolygonGroups.Add(PolygonGroupID, FPolygonGroupID(0));
		}
		RawMesh.RemapPolygonGroups(RemapPolygonGroups);
	}

	OutFlattenMaterial.UVChannel = INDEX_NONE;
}

void FMeshMergeUtilities::CreateMergedRawMeshes(FMeshMergeDataTracker& InDataTracker, const FMeshMergingSettings& InSettings, const TArray<UStaticMeshComponent*>& InStaticMeshComponentsToMerge, const TArray<UMaterialInterface*>& InUniqueMaterials, const TMap<UMaterialInterface*, UMaterialInterface*>& InCollapsedMaterialMap, const TMultiMap<FMeshLODKey, MaterialRemapPair>& InOutputMaterialsMap, bool bInMergeAllLODs, bool bInMergeMaterialData, const FVector& InMergedAssetPivot, TArray<FMeshDescription>& OutMergedRawMeshes) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshMergeUtilities::CreateMergedRawMeshes)

	if (bInMergeAllLODs)
	{
		OutMergedRawMeshes.AddDefaulted(InDataTracker.GetNumLODsForMergedMesh());
		for (TConstLODIndexIterator Iterator = InDataTracker.GetLODIndexIterator(); Iterator; ++Iterator)
		{
			// Find meshes for each lod
			const int32 LODIndex = *Iterator;
			FMeshDescription& MergedMesh = OutMergedRawMeshes[LODIndex];
			FStaticMeshAttributes(MergedMesh).Register();

			for (int32 ComponentIndex = 0; ComponentIndex < InStaticMeshComponentsToMerge.Num(); ++ComponentIndex)
			{
				int32 RetrievedLODIndex = LODIndex;
				FMeshDescription* RawMeshPtr = InDataTracker.TryFindRawMeshForLOD(ComponentIndex, RetrievedLODIndex);

				if (RawMeshPtr != nullptr)
				{
					InDataTracker.AddComponentToWedgeMapping(ComponentIndex, LODIndex, MergedMesh.VertexInstances().Num());

					FStaticMeshOperations::FAppendSettings AppendSettings;

					AppendSettings.PolygonGroupsDelegate = FAppendPolygonGroupsDelegate::CreateLambda([&bInMergeMaterialData, &InDataTracker, &InOutputMaterialsMap, &ComponentIndex, &LODIndex](const FMeshDescription& SourceMesh, FMeshDescription& TargetMesh, PolygonGroupMap& RemapPolygonGroups)
					{
						TPolygonGroupAttributesConstRef<FName> SourceImportedMaterialSlotNames = SourceMesh.PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName);
						TPolygonGroupAttributesRef<FName> TargetImportedMaterialSlotNames = TargetMesh.PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName);
						//Copy the polygon group
						if (bInMergeMaterialData)
						{
							FPolygonGroupID PolygonGroupID(0);
							if (!TargetMesh.PolygonGroups().IsValid(PolygonGroupID))
							{
								TargetMesh.CreatePolygonGroupWithID(PolygonGroupID);
								TargetImportedMaterialSlotNames[PolygonGroupID] = SourceMesh.PolygonGroups().IsValid(PolygonGroupID) ? SourceImportedMaterialSlotNames[PolygonGroupID] : FName(TEXT("DefaultMaterialName"));
							}
							for (FPolygonGroupID SourcePolygonGroupID : SourceMesh.PolygonGroups().GetElementIDs())
							{
								RemapPolygonGroups.Add(SourcePolygonGroupID, PolygonGroupID);
							}
						}
						else
						{
							TArray<SectionRemapPair> SectionMappings;
							InDataTracker.GetMappingsForMeshLOD(FMeshLODKey(ComponentIndex, LODIndex), SectionMappings);
							for (FPolygonGroupID SourcePolygonGroupID : SourceMesh.PolygonGroups().GetElementIDs())
							{
								// First map from original section index to unique material index
								int32 UniqueIndex = INDEX_NONE;
								// then map to the output material map, if any
								if (InOutputMaterialsMap.Num() > 0)
								{
									TArray<MaterialRemapPair> MaterialMappings;
									InOutputMaterialsMap.MultiFind(FMeshLODKey(ComponentIndex, LODIndex), MaterialMappings);
									for (MaterialRemapPair& Pair : MaterialMappings)
									{
										if (Pair.Key == SourcePolygonGroupID.GetValue())
										{
											UniqueIndex = Pair.Value;
											break;
										}
									}

									// Note that at this point UniqueIndex is NOT a material index, but a unique section index!
								}
								
								if(UniqueIndex == INDEX_NONE)
								{
									UniqueIndex = SourcePolygonGroupID.GetValue();
								}
								FPolygonGroupID TargetPolygonGroupID(UniqueIndex);
								if (!TargetMesh.PolygonGroups().IsValid(TargetPolygonGroupID))
								{
									while (TargetMesh.PolygonGroups().Num() <= UniqueIndex)
									{
										TargetPolygonGroupID = TargetMesh.CreatePolygonGroup();
									}
									check(TargetPolygonGroupID.GetValue() == UniqueIndex);
									TargetImportedMaterialSlotNames[TargetPolygonGroupID] = SourceImportedMaterialSlotNames[SourcePolygonGroupID];
								}
								RemapPolygonGroups.Add(SourcePolygonGroupID, TargetPolygonGroupID);
							}
						}
					});
					AppendSettings.bMergeVertexColor = InSettings.bBakeVertexDataToMesh;
					AppendSettings.MergedAssetPivot = InMergedAssetPivot;
					for (int32 ChannelIdx = 0; ChannelIdx < FStaticMeshOperations::FAppendSettings::MAX_NUM_UV_CHANNELS; ++ChannelIdx)
					{
						AppendSettings.bMergeUVChannels[ChannelIdx] = InDataTracker.DoesUVChannelContainData(ChannelIdx, LODIndex) && InSettings.OutputUVs[ChannelIdx] == EUVOutput::OutputChannel;
					}
					FStaticMeshOperations::AppendMeshDescription(*RawMeshPtr, MergedMesh, AppendSettings);
				}
			}

			//Cleanup the empty material to avoid empty section later
			TArray<FPolygonGroupID> PolygonGroupToRemove;
			for (FPolygonGroupID PolygonGroupID : MergedMesh.PolygonGroups().GetElementIDs())
			{
				if (MergedMesh.GetPolygonGroupPolygonIDs(PolygonGroupID).Num() < 1)
				{
					PolygonGroupToRemove.Add(PolygonGroupID);
					
				}
			}
			for (FPolygonGroupID PolygonGroupID : PolygonGroupToRemove)
			{
				MergedMesh.DeletePolygonGroup(PolygonGroupID);
			}
		}
	}
	else
	{	
		FMeshDescription& MergedMesh = OutMergedRawMeshes.AddDefaulted_GetRef();
		FStaticMeshAttributes(MergedMesh).Register();

		for (int32 ComponentIndex = 0; ComponentIndex < InStaticMeshComponentsToMerge.Num(); ++ComponentIndex)
		{
			int32 LODIndex = 0;
		
			FMeshDescription* RawMeshPtr = InDataTracker.FindRawMeshAndLODIndex(ComponentIndex, LODIndex);

			if (RawMeshPtr != nullptr)
			{
				FMeshDescription& RawMesh = *RawMeshPtr;

				const int32 TargetLODIndex = 0;
				InDataTracker.AddComponentToWedgeMapping(ComponentIndex, TargetLODIndex, MergedMesh.VertexInstances().Num());

				FStaticMeshOperations::FAppendSettings AppendSettings;

				AppendSettings.PolygonGroupsDelegate = FAppendPolygonGroupsDelegate::CreateLambda([&bInMergeMaterialData, &InDataTracker, &InOutputMaterialsMap, &ComponentIndex, &LODIndex](const FMeshDescription& SourceMesh, FMeshDescription& TargetMesh, PolygonGroupMap& RemapPolygonGroups)
				{
					TPolygonGroupAttributesConstRef<FName> SourceImportedMaterialSlotNames = SourceMesh.PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName);
					TPolygonGroupAttributesRef<FName> TargetImportedMaterialSlotNames = TargetMesh.PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName);
					//Copy the polygon group
					if (bInMergeMaterialData)
					{
						FPolygonGroupID PolygonGroupID(0);
						if (!TargetMesh.PolygonGroups().IsValid(PolygonGroupID))
						{
							TargetMesh.CreatePolygonGroupWithID(PolygonGroupID);
							TargetImportedMaterialSlotNames[PolygonGroupID] = SourceMesh.PolygonGroups().IsValid(PolygonGroupID) ? SourceImportedMaterialSlotNames[PolygonGroupID] : FName(TEXT("DefaultMaterialName"));
						}
						for (FPolygonGroupID SourcePolygonGroupID : SourceMesh.PolygonGroups().GetElementIDs())
						{
							RemapPolygonGroups.Add(SourcePolygonGroupID, PolygonGroupID);
						}
					}
					else
					{
						TArray<SectionRemapPair> SectionMappings;
						InDataTracker.GetMappingsForMeshLOD(FMeshLODKey(ComponentIndex, LODIndex), SectionMappings);
						for (FPolygonGroupID SourcePolygonGroupID : SourceMesh.PolygonGroups().GetElementIDs())
						{
							// First map from original section index to unique material index
							int32 UniqueIndex = INDEX_NONE;
							// then map to the output material map, if any
							if (InOutputMaterialsMap.Num() > 0)
							{
								TArray<MaterialRemapPair> MaterialMappings;
								InOutputMaterialsMap.MultiFind(FMeshLODKey(ComponentIndex, LODIndex), MaterialMappings);
								for (MaterialRemapPair& Pair : MaterialMappings)
								{
									if (Pair.Key == SourcePolygonGroupID.GetValue())
									{
										UniqueIndex = Pair.Value;
										break;
									}
								}

								// Note that at this point UniqueIndex is NOT a material index, but a unique section index!
							}
							
							//Fallback
							if(UniqueIndex == INDEX_NONE)
							{
								UniqueIndex = SourcePolygonGroupID.GetValue();
							}

							FPolygonGroupID TargetPolygonGroupID(UniqueIndex);
							if (!TargetMesh.PolygonGroups().IsValid(TargetPolygonGroupID))
							{
								while (TargetMesh.PolygonGroups().Num() <= UniqueIndex)
								{
									TargetPolygonGroupID = TargetMesh.CreatePolygonGroup();
								}
								check(TargetPolygonGroupID.GetValue() == UniqueIndex);
								TargetImportedMaterialSlotNames[TargetPolygonGroupID] = SourceImportedMaterialSlotNames[SourcePolygonGroupID];
							}
							RemapPolygonGroups.Add(SourcePolygonGroupID, TargetPolygonGroupID);
						}
					}
				});
				AppendSettings.bMergeVertexColor = InSettings.bBakeVertexDataToMesh;
				AppendSettings.MergedAssetPivot = InMergedAssetPivot;
				for (int32 ChannelIdx = 0; ChannelIdx < FStaticMeshOperations::FAppendSettings::MAX_NUM_UV_CHANNELS; ++ChannelIdx)
				{
					AppendSettings.bMergeUVChannels[ChannelIdx] = InDataTracker.DoesUVChannelContainData(ChannelIdx, LODIndex) && InSettings.OutputUVs[ChannelIdx] == EUVOutput::OutputChannel;
				}
				FStaticMeshOperations::AppendMeshDescription(*RawMeshPtr, MergedMesh, AppendSettings);
			}
		}
	}
}

void FMeshMergeUtilities::MergeComponentsToInstances(const TArray<UPrimitiveComponent*>& ComponentsToMerge, UWorld* World, ULevel* Level, const FMeshInstancingSettings& InSettings, bool bActuallyMerge /*= true*/, bool bReplaceSourceActors /* = false */, FText* OutResultsText /*= nullptr*/) const
{
	auto HasInstanceVertexColors = [](UStaticMeshComponent* StaticMeshComponent)
	{
		for (const FStaticMeshComponentLODInfo& CurrentLODInfo : StaticMeshComponent->LODData)
		{
			if(CurrentLODInfo.OverrideVertexColors != nullptr || CurrentLODInfo.PaintedVertices.Num() > 0)
			{
				return true;
			}
		}

		return false;
	};

	// Gather valid components
	TArray<UStaticMeshComponent*> ValidComponents;
	for(UPrimitiveComponent* ComponentToMerge : ComponentsToMerge)
	{
		if(UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(ComponentToMerge))
		{
			// Dont harvest from 'destination' actors
			if(StaticMeshComponent->GetOwner()->GetClass() != InSettings.ActorClassToUse.Get())
			{
				if( !InSettings.bSkipMeshesWithVertexColors || !HasInstanceVertexColors(StaticMeshComponent))
				{
					ValidComponents.Add(StaticMeshComponent);
				}
			}
		}
	}

	if(OutResultsText != nullptr)
	{
		*OutResultsText = LOCTEXT("InstanceMergePredictedResultsNone", "The current settings will not result in any instanced meshes being created");
	}

	if(ValidComponents.Num() > 0)
	{
		/** Helper struct representing a spawned ISMC */
		struct FComponentEntry
		{
			FComponentEntry(UStaticMeshComponent* InComponent)
			{
				StaticMesh = InComponent->GetStaticMesh();
				InComponent->GetUsedMaterials(Materials);
				bReverseCulling = InComponent->GetComponentTransform().ToMatrixWithScale().Determinant() < 0.0f;
				CollisionProfileName = InComponent->GetCollisionProfileName();
				CollisionEnabled = InComponent->GetCollisionEnabled();
				OriginalComponents.Add(InComponent);
			}

			bool operator==(const FComponentEntry& InOther) const
			{
				return 
					StaticMesh == InOther.StaticMesh &&
					Materials == InOther.Materials &&
					bReverseCulling == InOther.bReverseCulling && 
					CollisionProfileName == InOther.CollisionProfileName &&
					CollisionEnabled == InOther.CollisionEnabled;
			}

			UStaticMesh* StaticMesh;

			TArray<UMaterialInterface*> Materials;

			TArray<UStaticMeshComponent*> OriginalComponents;

			FName CollisionProfileName;

			bool bReverseCulling;

			ECollisionEnabled::Type CollisionEnabled;
		};

		/** Helper struct representing a spawned ISMC-containing actor */
		struct FActorEntry
		{
			FActorEntry(UStaticMeshComponent* InComponent, ULevel* InLevel)
				: MergedActor(nullptr)
			{
				// intersect with HLOD volumes if we have a level
				if(InLevel)
				{
					for (AActor* Actor : InLevel->Actors)
					{
						if (AHierarchicalLODVolume* HierarchicalLODVolume = Cast<AHierarchicalLODVolume>(Actor))
						{
							FBox BoundingBox = InComponent->Bounds.GetBox();
							FBox VolumeBox = HierarchicalLODVolume->GetComponentsBoundingBox(true);

							if (VolumeBox.IsInside(BoundingBox) || (HierarchicalLODVolume->bIncludeOverlappingActors && VolumeBox.Intersect(BoundingBox)))
							{
								HLODVolume = HierarchicalLODVolume;
								break;
							}
						}
					}
				}
			}

			bool operator==(const FActorEntry& InOther) const
			{
				return HLODVolume == InOther.HLODVolume;
			}

			AActor* MergedActor;
			AHierarchicalLODVolume* HLODVolume;
			TArray<FComponentEntry> ComponentEntries;
		};

		// Gather a list of components to merge
		TArray<FActorEntry> ActorEntries;
		for(UStaticMeshComponent* StaticMeshComponent : ValidComponents)
		{
			int32 ActorEntryIndex = ActorEntries.AddUnique(FActorEntry(StaticMeshComponent, InSettings.bUseHLODVolumes ? Level : nullptr));
			FActorEntry& ActorEntry = ActorEntries[ActorEntryIndex];

			FComponentEntry ComponentEntry(StaticMeshComponent);

			if(FComponentEntry* ExistingComponentEntry = ActorEntry.ComponentEntries.FindByKey(ComponentEntry))
			{
				ExistingComponentEntry->OriginalComponents.Add(StaticMeshComponent);
			}
			else
			{
				ActorEntry.ComponentEntries.Add(ComponentEntry);
			}
		}

		// Filter by component count
		for(FActorEntry& ActorEntry : ActorEntries)
		{
			ActorEntry.ComponentEntries = ActorEntry.ComponentEntries.FilterByPredicate([&InSettings](const FComponentEntry& InEntry)
			{
				return InEntry.OriginalComponents.Num() >= InSettings.InstanceReplacementThreshold;
			});
		}

		// Remove any empty actor entries
		ActorEntries.RemoveAll([](const FActorEntry& ActorEntry){ return ActorEntry.ComponentEntries.Num() == 0; });

		int32 TotalComponentCount = 0;
		TArray<AActor*> ActorsToCleanUp;
		for(FActorEntry& ActorEntry : ActorEntries)
		{
			for(const FComponentEntry& ComponentEntry : ActorEntry.ComponentEntries)
			{
				TotalComponentCount++;
				for(UStaticMeshComponent* OriginalComponent : ComponentEntry.OriginalComponents)
				{
					if(AActor* OriginalActor = OriginalComponent->GetOwner())
					{
						ActorsToCleanUp.AddUnique(OriginalActor);
					}
				}
			}
		}

		if(ActorEntries.Num() > 0)
		{
			if(OutResultsText != nullptr)
			{
				*OutResultsText = FText::Format(LOCTEXT("InstanceMergePredictedResults", "The current settings will result in {0} instanced static mesh components ({1} actors will be replaced)"), FText::AsNumber(TotalComponentCount), FText::AsNumber(ActorsToCleanUp.Num()));
			}
			
			if(bActuallyMerge)
			{
				// Create our actors
				const FScopedTransaction Transaction(LOCTEXT("PlaceInstancedActors", "Place Instanced Actor(s)"));
				Level->Modify(false);

 				FActorSpawnParameters Params;
 				Params.OverrideLevel = Level;

				// We now have the set of component data we want to apply
				for(FActorEntry& ActorEntry : ActorEntries)
				{
					ActorEntry.MergedActor = World->SpawnActor<AActor>(InSettings.ActorClassToUse.Get(), Params);

					for(const FComponentEntry& ComponentEntry : ActorEntry.ComponentEntries)
					{
						UInstancedStaticMeshComponent* NewComponent = nullptr;

						NewComponent = (UInstancedStaticMeshComponent*)ActorEntry.MergedActor->FindComponentByClass(InSettings.ISMComponentToUse.Get());

						if (NewComponent && NewComponent->PerInstanceSMData.Num() > 0)
						{
							NewComponent = nullptr;
						}

						if (NewComponent == nullptr)
						{
							NewComponent = NewObject<UInstancedStaticMeshComponent>(ActorEntry.MergedActor, InSettings.ISMComponentToUse.Get(), NAME_None, RF_Transactional);
							NewComponent->bHasPerInstanceHitProxies = true;
						
							if (ActorEntry.MergedActor->GetRootComponent())
							{
								// Attach to root if we already have one
								NewComponent->AttachToComponent(ActorEntry.MergedActor->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
							}
							else
							{
								// Make a new root if we dont have a root already
								ActorEntry.MergedActor->SetRootComponent(NewComponent);
							}

							// Take 'instanced' ownership so it persists with this actor
							ActorEntry.MergedActor->RemoveOwnedComponent(NewComponent);
							NewComponent->CreationMethod = EComponentCreationMethod::Instance;
							ActorEntry.MergedActor->AddOwnedComponent(NewComponent);

						}

						NewComponent->SetStaticMesh(ComponentEntry.StaticMesh);
						for(int32 MaterialIndex = 0; MaterialIndex < ComponentEntry.Materials.Num(); ++MaterialIndex)
						{
							NewComponent->SetMaterial(MaterialIndex, ComponentEntry.Materials[MaterialIndex]);
						}
						NewComponent->SetReverseCulling(ComponentEntry.bReverseCulling);
						NewComponent->SetCollisionProfileName(ComponentEntry.CollisionProfileName);
						NewComponent->SetCollisionEnabled(ComponentEntry.CollisionEnabled);
						NewComponent->SetMobility(EComponentMobility::Static);

						FISMComponentBatcher ISMComponentBatcher;
						ISMComponentBatcher.Append(ComponentEntry.OriginalComponents);
						ISMComponentBatcher.InitComponent(NewComponent);						

						NewComponent->RegisterComponent();
					}

					World->UpdateCullDistanceVolumes(ActorEntry.MergedActor);
				}

				// Now clean up our original actors
				for(AActor* ActorToCleanUp : ActorsToCleanUp)
				{
					if (bReplaceSourceActors)
					{
						ActorToCleanUp->Destroy();
					}
					else
					{
						ActorToCleanUp->Modify();
						ActorToCleanUp->bIsEditorOnlyActor = true;
						ActorToCleanUp->SetHidden(true);
						ActorToCleanUp->bHiddenEd = true;
						ActorToCleanUp->SetIsTemporarilyHiddenInEditor(true);
					}
				}

				// pop a toast allowing selection
				auto SelectActorsLambda = [ActorEntries]()
				{ 
					GEditor->GetSelectedActors()->Modify();
					GEditor->GetSelectedActors()->BeginBatchSelectOperation();
					GEditor->SelectNone(false, true, false);

					for(const FActorEntry& ActorEntry : ActorEntries)
					{
						GEditor->SelectActor(ActorEntry.MergedActor, true, false, true);
					}

					GEditor->GetSelectedActors()->EndBatchSelectOperation();
				};

				// Always change selection if we removed the source actors,
				// Otherwise, allow selection change through notification
				if (bReplaceSourceActors)
				{
					SelectActorsLambda();
				}
				else
				{
					FNotificationInfo NotificationInfo(FText::Format(LOCTEXT("CreatedInstancedActorsMessage", "Created {0} Instanced Actor(s)"), FText::AsNumber(ActorEntries.Num())));
					NotificationInfo.Hyperlink = FSimpleDelegate::CreateLambda(SelectActorsLambda);
					NotificationInfo.HyperlinkText = LOCTEXT("SelectActorsHyperlink", "Select Actors");
					NotificationInfo.ExpireDuration = 5.0f;

					FSlateNotificationManager::Get().AddNotification(NotificationInfo);
				}
			}
		}
	}
}

UMaterialInterface* FMeshMergeUtilities::CreateProxyMaterial(const FString &InBasePackageName, FString MergedAssetPackageName, UMaterialInterface* InBaseMaterial, UPackage* InOuter, const FMeshMergingSettings &InSettings, const FFlattenMaterial& OutMaterial, TArray<UObject *>& OutAssetsToSync, FMaterialUpdateContext* InMaterialUpdateContext) const
{
	// Create merged material asset
	FString MaterialAssetName;
	FString MaterialPackageName;
	if (InBasePackageName.IsEmpty())
	{
		MaterialAssetName = FPackageName::GetShortName(MergedAssetPackageName);
		MaterialPackageName = FPackageName::GetLongPackagePath(MergedAssetPackageName) + TEXT("/");
	}
	else
	{
		MaterialAssetName = FPackageName::GetShortName(InBasePackageName);
		MaterialPackageName = FPackageName::GetLongPackagePath(InBasePackageName) + TEXT("/");
	}

	UPackage* MaterialPackage = InOuter;
	if (MaterialPackage == nullptr)
	{
		MaterialPackage = CreatePackage( *(MaterialPackageName + MaterialAssetName));
		check(MaterialPackage);
		MaterialPackage->FullyLoad();
		MaterialPackage->Modify();
	}

	UMaterialInstanceConstant* MergedMaterial = FMaterialUtilities::CreateFlattenMaterialInstance(MaterialPackage, InSettings.MaterialSettings, InBaseMaterial, OutMaterial, MaterialPackageName, MaterialAssetName, OutAssetsToSync, InMaterialUpdateContext);
	// Set material static lighting usage flag if project has static lighting enabled
	if (IsStaticLightingAllowed())
	{
		MergedMaterial->CheckMaterialUsage(MATUSAGE_StaticLighting);
	}

	return MergedMaterial;
}

void FMeshMergeUtilities::RetrievePhysicsData(const TArray<UPrimitiveComponent*>& ComponentsToMerge, TArray<FKAggregateGeom>& InOutPhysicsGeometry, UBodySetup*& OutBodySetupSource) const
{
	InOutPhysicsGeometry.AddDefaulted(ComponentsToMerge.Num());
	for (int32 ComponentIndex = 0, PhysicsGeometryIndex = 0; ComponentIndex < ComponentsToMerge.Num(); ++ComponentIndex)
	{
		UPrimitiveComponent* PrimComp = ComponentsToMerge[ComponentIndex];
		UBodySetup* BodySetup = nullptr;
		FTransform ComponentToWorld = FTransform::Identity;

		auto ExtractPhysicGeometry = [&BodySetup, &ComponentToWorld, &OutBodySetupSource, &InOutPhysicsGeometry, PrimComp](int32 PhysicsIndex) {
				USplineMeshComponent* SplineMeshComponent = Cast<USplineMeshComponent>(PrimComp);
				FMeshMergeHelpers::ExtractPhysicsGeometry(BodySetup, ComponentToWorld, SplineMeshComponent != nullptr, InOutPhysicsGeometry[PhysicsIndex]);
				if (SplineMeshComponent)
				{
					FMeshMergeHelpers::PropagateSplineDeformationToPhysicsGeometry(SplineMeshComponent, InOutPhysicsGeometry[PhysicsIndex]);
				}

				// We will use first valid BodySetup as a source of physics settings
				if (OutBodySetupSource == nullptr)
				{
					OutBodySetupSource = BodySetup;
				}
			};

		if (UInstancedStaticMeshComponent* ISMComp = Cast<UInstancedStaticMeshComponent>(PrimComp))
		{
			const int32 NumberOfInstances = ISMComp->PerInstanceSMData.Num();
			const UStaticMesh* SrcMesh = ISMComp->GetStaticMesh();
			
			if (NumberOfInstances > 1)
			{
				InOutPhysicsGeometry.AddDefaulted(NumberOfInstances - 1);
			}

			if (SrcMesh)
			{
				BodySetup = SrcMesh->GetBodySetup();
			}

			for (const FInstancedStaticMeshInstanceData& InstanceData : ISMComp->PerInstanceSMData)
			{
				ComponentToWorld = FTransform(InstanceData.Transform) * ISMComp->GetComponentToWorld();
				ExtractPhysicGeometry(PhysicsGeometryIndex++);
			}
		}
		else if (UStaticMeshComponent* StaticMeshComp = Cast<UStaticMeshComponent>(PrimComp))
		{
			UStaticMesh* SrcMesh = StaticMeshComp->GetStaticMesh();
			if (SrcMesh)
			{
				BodySetup = SrcMesh->GetBodySetup();
			}
			ComponentToWorld = StaticMeshComp->GetComponentToWorld();
			ExtractPhysicGeometry(PhysicsGeometryIndex++);
		}
		else if (UShapeComponent* ShapeComp = Cast<UShapeComponent>(PrimComp))
		{
			BodySetup = ShapeComp->GetBodySetup();
			ComponentToWorld = ShapeComp->GetComponentToWorld();
			ExtractPhysicGeometry(PhysicsGeometryIndex++);
		}

	}
}

#undef LOCTEXT_NAMESPACE // "MeshMergeUtils"
