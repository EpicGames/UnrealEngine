// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMeshMergeUtilities.h"
#include "SceneTypes.h"
#include "StaticMeshComponentAdapter.h"
#include "MaterialShared.h"

class FMeshMergeDataTracker;
class FMaterialUpdateContext;
struct FMeshLODKey;

typedef TPair<uint32, uint32> MaterialRemapPair;

/**
 * Mesh Merge Utilities
 */
class FMeshMergeUtilities : public IMeshMergeUtilities
{
public:	
	friend class FProxyGenerationProcessor;
	FMeshMergeUtilities();
	virtual ~FMeshMergeUtilities() override;
	virtual void BakeMaterialsForComponent(TArray<TWeakObjectPtr<UObject>>& OptionObjects, IMaterialBakingAdapter* Adapter) const override;
	virtual void BakeMaterialsForComponent(UStaticMeshComponent* StaticMeshComponent) const override;
	virtual void BakeMaterialsForComponent(USkeletalMeshComponent* SkeletalMeshComponent) const override;
	virtual void BakeMaterialsForMesh(UStaticMesh* Mesh) const override;
	virtual void MergeComponentsToStaticMesh(const TArray<UPrimitiveComponent*>& ComponentsToMerge, UWorld* World, const FMeshMergingSettings& InSettings, UMaterialInterface* InBaseMaterial, UPackage* InOuter, const FString& InBasePackageName, TArray<UObject*>& OutAssetsToSync, FVector& OutMergedActorLocation, const float ScreenSize, bool bSilent /*= false*/) const;
	virtual void MergeComponentsToInstances(const TArray<UPrimitiveComponent*>& ComponentsToMerge, UWorld* World, ULevel* Level, const FMeshInstancingSettings& InSettings, bool bActuallyMerge = true, bool bReplaceSourceActors = false, FText* OutResultsText = nullptr) const override;

	virtual void CreateProxyMesh(const TArray<AActor*>& InActors, const struct FMeshProxySettings& InMeshProxySettings, UMaterialInterface* InBaseMaterial, UPackage* InOuter, const FString& InProxyBasePackageName, const FGuid InGuid, const FCreateProxyDelegate& InProxyCreatedDelegate, const bool bAllowAsync = false, const float ScreenSize = 1.0f) const override;
	virtual void CreateProxyMesh(const TArray<UStaticMeshComponent*>& InStaticMeshComps, const struct FMeshProxySettings& InMeshProxySettings, UMaterialInterface* InBaseMaterial, UPackage* InOuter, const FString& InProxyBasePackageName, const FGuid InGuid, const FCreateProxyDelegate& InProxyCreatedDelegate, const bool bAllowAsync = false, const float ScreenSize = 1.0f) const override;
	virtual void CreateProxyMesh(const TArray<AActor*>& InActors, const struct FMeshProxySettings& InMeshProxySettings, UPackage* InOuter, const FString& InProxyBasePackageName, const FGuid InGuid, const FCreateProxyDelegate& InProxyCreatedDelegate, const bool bAllowAsync = false, const float ScreenSize = 1.0f) const override;
	virtual void CreateProxyMesh(const TArray<UStaticMeshComponent*>& InStaticMeshComps, const struct FMeshProxySettings& InMeshProxySettings, UPackage* InOuter, const FString& InProxyBasePackageName, const FGuid InGuid, const FCreateProxyDelegate& InProxyCreatedDelegate, const bool bAllowAsync = false, const float ScreenSize = 1.0f) const override;

	virtual void RetrieveMeshDescription(const UStaticMeshComponent* InStaticMeshComponent, int32 LODIndex, FMeshDescription& InOutMeshDescription, bool bPropagateVertexColours) const override;
	virtual void RetrieveMeshDescription(const USkeletalMeshComponent* InSkeletalMeshComponent, int32 LODIndex, FMeshDescription& InOutMeshDescription, bool bPropagateVertexColours) const override;
	virtual void RetrieveMeshDescription(const UStaticMesh* InStaticMesh, int32 LODIndex, FMeshDescription& InOutMeshDescription) const override;

	virtual void RetrievePhysicsData(const TArray<UPrimitiveComponent*>& InComponents, TArray<FKAggregateGeom>& InOutPhysicsGeometry, UBodySetup*& OutBodySetupSource) const override;

	virtual void RegisterExtension(IMeshMergeExtension* InExtension) override;
	virtual void UnregisterExtension(IMeshMergeExtension* InExtension) override;	
protected:
	/** Creates a proxy material instance at givne path and name */
	UMaterialInterface* CreateProxyMaterial(const FString &InBasePackageName, FString MergedAssetPackageName, UMaterialInterface* InBaseMaterial, UPackage* InOuter, const FMeshMergingSettings &InSettings, const FFlattenMaterial& OutMaterial, TArray<UObject *>& OutAssetsToSync, FMaterialUpdateContext* InMaterialUpdateContext = nullptr) const;
	/** Merges flattened material into atlas textures */
	void MergeFlattenedMaterials(TArray<struct FFlattenMaterial>& InMaterialList, int32 InGutter, FFlattenMaterial& OutMergedMaterial, TArray<FUVOffsetScalePair>& OutUVTransforms) const;
	/** Merges flattened material into binned textures */
	void FlattenBinnedMaterials(TArray<struct FFlattenMaterial>& InMaterialList, const TArray<FBox2D>& InMaterialBoxes, int32 InGutter, bool bCopyOnlyMaskedPixels, FFlattenMaterial& OutMergedMaterial, TArray<FUVOffsetScalePair>& OutUVTransforms) const;

	void CreateMergedMaterial(FMeshMergeDataTracker& InDataTracker, const FMeshMergingSettings& InSettings, const TArray<UStaticMeshComponent*>& InStaticMeshComponentsToMerge, TArray<FStaticMeshComponentAdapter>& InAdapters, const TArray<UMaterialInterface*>& InUniqueMaterials, const TMap<UMaterialInterface*, UMaterialInterface*>& InCollapsedMaterialMap, TMultiMap<FMeshLODKey, MaterialRemapPair>& InOutputMaterialsMap, bool bInMergeAllLODs, bool bInMergeMaterialData, const FVector& InMergedAssetPivot, FFlattenMaterial& OutFlattenMaterial) const;

	/** Helper function to create the final merged raw meshes */
	void CreateMergedRawMeshes(FMeshMergeDataTracker& InDataTracker, const FMeshMergingSettings& InSettings, const TArray<UStaticMeshComponent*>& InStaticMeshComponentsToMerge, const TArray<UMaterialInterface*>& InUniqueMaterials, const TMap<UMaterialInterface*, UMaterialInterface*>& InCollapsedMaterialMap, const TMultiMap<FMeshLODKey, MaterialRemapPair>& InOutputMaterialsMap, bool bInMergeAllLODs, bool bInMergeMaterialData, const FVector& InMergedAssetPivot, TArray<FMeshDescription>& OutMergedRawMeshes) const;
protected:
	/** Flattens out emissive scale across all flatten material instances */
	float FlattenEmissivescale(TArray<struct FFlattenMaterial>& InMaterialList) const;
	/** Populates material options object from legacy material proxy settings  */
	UMaterialOptions* PopulateMaterialOptions(const FMaterialProxySettings& MaterialSettings) const;
	/** Populates a single property entry with correct material baking settings */
	void PopulatePropertyEntry(const FMaterialProxySettings& MaterialSettings, EMaterialProperty MaterialProperty, struct FPropertyEntry& InOutPropertyEntry) const;
	/** Copies part (box) from a texture to another texture */
	void CopyTextureRect(const FColor* Src, const FIntPoint& SrcSize, FColor* Dst, const FIntPoint& DstSize, const FIntPoint& DstPos, bool bCopyOnlyMaskedPixels = false) const;
	/** Sets a part (box) on a texture to ColorValue */
	void SetTextureRect(const FColor& ColorValue, const FIntPoint& SrcSize, FColor* Dst, const FIntPoint& DstSize, const FIntPoint& DstPos) const;
	/** Conditionally resizes the source data into OutImage */
	FIntPoint ConditionalImageResize(const FIntPoint& SrcSize, const FIntPoint& DesiredSize, TArray<FColor>& InOutImage, bool bLinearSpace) const;
	/** Converts bake output structure data to flatten material format */
	void ConvertOutputToFlatMaterials(const TArray<FBakeOutput>& BakeOutputs, const TArray<FMaterialData>& MaterialData, TArray<FFlattenMaterial> &FlattenedMaterials) const;
	/** Transfer bake output structure data to flatten material format.
	  * @warning This is a destructive operation for InOutBakeOutputs
	  */
	void TransferOutputToFlatMaterials(const TArray<FMaterialData>& InMaterialData, TArray<FBakeOutput>& InOutBakeOutputs, TArray<FFlattenMaterial> &OutFlattenedMaterials) const;

	/** Converts material property enum value to flatten material property */
	EFlattenMaterialProperties ToFlattenProperty(EMaterialProperty MaterialProperty) const;
private:
	FProxyGenerationProcessor* Processor;
	FDelegateHandle ModuleLoadedDelegateHandle;

	TArray<IMeshMergeExtension*> MeshMergeExtensions;
};
