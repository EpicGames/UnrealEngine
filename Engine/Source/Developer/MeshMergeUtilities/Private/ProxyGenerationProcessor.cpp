// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProxyGenerationProcessor.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "MaterialUtilities.h"
#include "MeshMergeUtilities.h"
#include "IMeshMergeExtension.h"
#include "IMeshReductionInterfaces.h"
#include "IMeshReductionManagerModule.h"
#include "Modules/ModuleManager.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshResources.h"
#include "Stats/Stats.h"
#include "Algo/ForEach.h"

#if WITH_EDITOR
#include "Editor.h"
#include "MeshMergeHelpers.h"
#include "ObjectCacheEventSink.h"
#include "Materials/MaterialInstanceConstant.h"
#endif // WITH_EDITOR

FProxyGenerationProcessor::FProxyGenerationProcessor(const FMeshMergeUtilities* InOwner)
	: Owner(InOwner)
{
#if WITH_EDITOR
	FEditorDelegates::MapChange.AddRaw(this, &FProxyGenerationProcessor::OnMapChange);
	FEditorDelegates::NewCurrentLevel.AddRaw(this, &FProxyGenerationProcessor::OnNewCurrentLevel);
	
	IMeshReductionManagerModule& Module = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface");
	Module.OnModuleConfigurationCompleted(FSimpleDelegate::CreateRaw(this, &FProxyGenerationProcessor::OnModuleConfigurationCompleted));
#endif // WITH_EDITOR
}

FProxyGenerationProcessor::~FProxyGenerationProcessor()
{
#if WITH_EDITOR
	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::NewCurrentLevel.RemoveAll(this);
#endif // WITH_EDITOR
}

void FProxyGenerationProcessor::OnModuleConfigurationCompleted()
{
#if WITH_EDITOR
	IMeshReductionManagerModule& Module = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface");
	IMeshMerging* MeshMerging = Module.GetMeshMergingInterface();
	if (!MeshMerging)
	{
		UE_LOGF(LogMeshMerging, Log, "No automatic mesh merging module available");
	}
	else
	{
		MeshMerging->CompleteDelegate.BindRaw(this, &FProxyGenerationProcessor::ProxyGenerationComplete);
		MeshMerging->FailedDelegate.BindRaw(this, &FProxyGenerationProcessor::ProxyGenerationFailed);
	}

	IMeshMerging* DistributedMeshMerging = Module.GetDistributedMeshMergingInterface();
	if (!DistributedMeshMerging)
	{
		UE_LOGF(LogMeshMerging, Log, "No distributed automatic mesh merging module available");
	}
	else
	{
		DistributedMeshMerging->CompleteDelegate.BindRaw(this, &FProxyGenerationProcessor::ProxyGenerationComplete);
		DistributedMeshMerging->FailedDelegate.BindRaw(this, &FProxyGenerationProcessor::ProxyGenerationFailed);
	}
#endif
}

void FProxyGenerationProcessor::AddProxyJob(FGuid InJobGuid, FMergeCompleteData* InCompleteData)
{
	FScopeLock Lock(&StateLock);
	ProxyMeshJobs.Add(InJobGuid, InCompleteData);
}

bool FProxyGenerationProcessor::Tick(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FProxyGenerationProcessor_Tick);

	FScopeLock Lock(&StateLock);
	for (const auto& Entry : ToProcessJobDataMap)
	{
		FGuid JobGuid = Entry.Key;
		FProxyGenerationData* Data = Entry.Value;

		// Process the job
		ProcessJob(JobGuid, Data);

		// Data retrieved so can now remove the job from the map
		ProxyMeshJobs.Remove(JobGuid);
		delete Data->MergeData;
		delete Data;
	}

	ToProcessJobDataMap.Reset();

	return true;
}

void FProxyGenerationProcessor::ProxyGenerationComplete(FMeshDescription& OutProxyMesh, struct FFlattenMaterial& OutMaterial, const FGuid OutJobGUID)
{
	FScopeLock Lock(&StateLock);
	FMergeCompleteData** FindData = ProxyMeshJobs.Find(OutJobGUID);
	if (FindData && *FindData)
	{
		FMergeCompleteData* Data = *FindData;

		FProxyGenerationData* GenerationData = new FProxyGenerationData();
		GenerationData->Material = OutMaterial;
		GenerationData->RawMesh = OutProxyMesh;
		GenerationData->MergeData = Data;

		ToProcessJobDataMap.Add(OutJobGUID, GenerationData);
	}
}

void FProxyGenerationProcessor::ProxyGenerationFailed(const FGuid OutJobGUID, const FString& ErrorMessage)
{
	FScopeLock Lock(&StateLock);
	FMergeCompleteData** FindData = ProxyMeshJobs.Find(OutJobGUID);
	if (FindData && *FindData)
	{
		UE_LOGF(LogMeshMerging, Log, "Failed to generate proxy mesh for cluster %ls, %ls", *(*FindData)->ProxyBasePackageName, *ErrorMessage);
		ProxyMeshJobs.Remove(OutJobGUID);
		
		TArray<UObject*> OutAssetsToSync;
		(*FindData)->CallbackDelegate.ExecuteIfBound(OutJobGUID, OutAssetsToSync);
	}
}

void FProxyGenerationProcessor::OnMapChange(uint32 MapFlags)
{
	ClearProcessingData();
}

void FProxyGenerationProcessor::OnNewCurrentLevel()
{
	ClearProcessingData();
}

void FProxyGenerationProcessor::ClearProcessingData()
{
	FScopeLock Lock(&StateLock);
	ProxyMeshJobs.Empty();
	ToProcessJobDataMap.Empty();
}

void FProxyGenerationProcessor::ProcessJob(const FGuid& JobGuid, FProxyGenerationData* Data)
{
	TArray<UObject*> OutAssetsToSync;
	const FString AssetBaseName = FPackageName::GetShortName(Data->MergeData->ProxyBasePackageName);
	const FString AssetBasePath = Data->MergeData->InOuter ? TEXT("") : FPackageName::GetLongPackagePath(Data->MergeData->ProxyBasePackageName) + TEXT("/");

	UMaterialInstanceConstant* ProxyMaterial = nullptr;

	if (!Data->RawMesh.IsEmpty())
	{
		Data->MergeData->InProxySettings.MaterialSettings.ResolveTextureSize(Data->RawMesh);

		// Don't recreate render states with the material update context as we will manually do it through
		// the FStaticMeshComponentRecreateRenderStateContext below
		FMaterialUpdateContext MaterialUpdateContext(FMaterialUpdateContext::EOptions::Default & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);

		// Retrieve flattened material data
		FFlattenMaterial& FlattenMaterial = Data->Material;

		// Resize flattened material
		FMaterialUtilities::ResizeFlattenMaterial(FlattenMaterial, Data->MergeData->InProxySettings);

		// Optimize flattened material
		FMaterialUtilities::OptimizeFlattenMaterial(FlattenMaterial);

		// Create a new proxy material instance
		ProxyMaterial = FMaterialUtilities::CreateFlattenMaterialInstance(Data->MergeData->InOuter, Data->MergeData->InProxySettings.MaterialSettings, Data->MergeData->BaseMaterial, FlattenMaterial, AssetBasePath, AssetBaseName, OutAssetsToSync, &MaterialUpdateContext);

		for (IMeshMergeExtension* Extension : Owner->MeshMergeExtensions)
		{
			Extension->OnCreatedProxyMaterial(Data->MergeData->StaticMeshComponents, ProxyMaterial);
		}

		// Set material static lighting usage flag if project has static lighting enabled
		if (IsStaticLightingAllowed())
		{
			ProxyMaterial->CheckMaterialUsage(MATUSAGE_StaticLighting);
		}
	}

	// Construct proxy static mesh
	UPackage* MeshPackage = Data->MergeData->InOuter;
	FString MeshAssetName = TEXT("SM_") + AssetBaseName;
	if (MeshPackage == nullptr)
	{
		MeshPackage = CreatePackage( *(AssetBasePath + MeshAssetName));
		MeshPackage->FullyLoad();
		MeshPackage->Modify();
	}

	UStaticMesh* OldStaticMesh = FindObject<UStaticMesh>(MeshPackage, *MeshAssetName);

	FStaticMeshComponentRecreateRenderStateContext RecreateRenderStateContext(OldStaticMesh);

	UStaticMesh* StaticMesh = NewObject<UStaticMesh>(MeshPackage, FName(*MeshAssetName), RF_Public | RF_Standalone);
	StaticMesh->InitResources();

	FString OutputPath = StaticMesh->GetPathName();

	// make sure it has a new lighting guid
	StaticMesh->SetLightingGuid();

	// Set it to use textured lightmaps. Note that Build Lighting will do the error-checking (texcoordindex exists for all LODs, etc).
	StaticMesh->SetLightMapResolution(Data->MergeData->InProxySettings.LightMapResolution);
	StaticMesh->SetLightMapCoordinateIndex(1);

	// Ray tracing support
	StaticMesh->bSupportRayTracing = Data->MergeData->InProxySettings.bSupportRayTracing;

	FStaticMeshSourceModel& SrcModel = StaticMesh->AddSourceModel();
	/*Don't allow the engine to recalculate normals*/
	SrcModel.BuildSettings.bRecomputeNormals = false;
	SrcModel.BuildSettings.bRecomputeTangents = false;
	SrcModel.BuildSettings.bComputeWeightedNormals = true;
	SrcModel.BuildSettings.bRemoveDegenerates = true;
	SrcModel.BuildSettings.bUseHighPrecisionTangentBasis = false;
	SrcModel.BuildSettings.bUseFullPrecisionUVs = false;
	SrcModel.BuildSettings.bGenerateLightmapUVs = Data->MergeData->InProxySettings.bGenerateLightmapUVs;
	SrcModel.BuildSettings.bBuildReversedIndexBuffer = false;
	if (!Data->MergeData->InProxySettings.bAllowDistanceField)
	{
		SrcModel.BuildSettings.DistanceFieldResolutionScale = 0.0f;
	}

	const bool bContainsImposters = Data->MergeData->ImposterComponents.Num() > 0;
	FBox ImposterBounds(EForceInit::ForceInit);

	TPolygonGroupAttributesConstRef<FName> PolygonGroupMaterialSlotName = FStaticMeshAttributes(Data->RawMesh).GetPolygonGroupMaterialSlotNames();

	auto RemoveVertexColorAndCommitMeshDescription = [&StaticMesh, &Data, &ProxyMaterial, &PolygonGroupMaterialSlotName]()
	{
		if (!Data->MergeData->InProxySettings.bAllowVertexColors)
		{
			//We cannot remove the vertex color with the mesh description so we assign a white value to all color
			TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = FStaticMeshAttributes(Data->RawMesh).GetVertexInstanceColors();
			//set all value to white
			for (const FVertexInstanceID VertexInstanceID : Data->RawMesh.VertexInstances().GetElementIDs())
			{
				VertexInstanceColors[VertexInstanceID] = FVector4f(1.0f, 1.0f, 1.0f);
			}
		}

		//Commit the FMeshDescription to the source model we just created
		int32 SourceModelIndex = StaticMesh->GetNumSourceModels() - 1;
		FMeshDescription* MeshDescription = StaticMesh->CreateMeshDescription(SourceModelIndex, Data->RawMesh);
		if (ensure(MeshDescription))
		{
			if (ProxyMaterial)
			{
				// Make sure the Proxy material have a valid ImportedMaterialSlotName
				//The proxy material must be add only once and is always the first slot of the HLOD mesh
				FStaticMaterial NewMaterial(ProxyMaterial);
				if (MeshDescription->PolygonGroups().Num() > 0)
				{
					NewMaterial.ImportedMaterialSlotName = PolygonGroupMaterialSlotName[MeshDescription->PolygonGroups().GetFirstValidID()];
				}
				StaticMesh->GetStaticMaterials().Add(NewMaterial);
			}

			UStaticMesh::FCommitMeshDescriptionParams CommitParams;
			CommitParams.bUseHashAsGuid = true;
			StaticMesh->CommitMeshDescription(SourceModelIndex, CommitParams);
		}
	};

	if (bContainsImposters)
	{
		TArray<UMaterialInterface*> ImposterMaterials;

		// Merge imposter meshes to rawmesh
		// The base material index is always one here as we assume we only have one HLOD material
		FMeshMergeHelpers::MergeImpostersToMesh(Data->MergeData->ImposterComponents, Data->RawMesh, FVector::ZeroVector, 1, ImposterMaterials);

		for (const UStaticMeshComponent* Component : Data->MergeData->ImposterComponents)
		{
			if (Component->GetStaticMesh())
			{
				ImposterBounds += Component->GetStaticMesh()->GetBoundingBox().TransformBy(Component->GetComponentToWorld());
			}
		}
		RemoveVertexColorAndCommitMeshDescription();

		for (UMaterialInterface* Material : ImposterMaterials)
		{
			//Set the ImportedMaterialSlotName in each imposter material
			FStaticMaterial NewMaterial(Material);
			if (Data->RawMesh.PolygonGroups().Num() > 0)
			{
				NewMaterial.ImportedMaterialSlotName = PolygonGroupMaterialSlotName[Data->RawMesh.PolygonGroups().GetFirstValidID()];
			}
			StaticMesh->GetStaticMaterials().Add(NewMaterial);
		}

		// Ensure the new mesh is not referencing non standalone materials
		FMeshMergeHelpers::FixupNonStandaloneMaterialReferences(StaticMesh);
	}
	else
	{
		RemoveVertexColorAndCommitMeshDescription();
	}

	// Nanite settings
	StaticMesh->SetNaniteSettings(Data->MergeData->InProxySettings.NaniteSettings);

	//Set the Imported version before calling the build
	StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);

	// setup section info map
	TPolygonGroupAttributesConstRef<FName> PolygonGroupImportedMaterialSlotNames = Data->RawMesh.PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName);
	TArray<int32> UniqueMaterialIndices;
	for (const FPolygonGroupID PolygonGroupID : Data->RawMesh.PolygonGroups().GetElementIDs())
	{
		int32 PolygonGroupMaterialIndex = PolygonGroupID.GetValue();
		FName PolygonGroupName = PolygonGroupImportedMaterialSlotNames[PolygonGroupID];
		if (PolygonGroupName != NAME_None)
		{
			for (int32 MaterialIndex = 0; MaterialIndex < StaticMesh->GetStaticMaterials().Num(); ++MaterialIndex)
			{
				if (StaticMesh->GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName == PolygonGroupName)
				{
					PolygonGroupMaterialIndex = MaterialIndex;
					break;
				}
			}
		}
		if (!StaticMesh->GetStaticMaterials().IsValidIndex(PolygonGroupMaterialIndex))
		{
			PolygonGroupMaterialIndex = 0;
		}
		UniqueMaterialIndices.AddUnique(PolygonGroupMaterialIndex);
	}

	int32 SectionIndex = 0;
	for (int32 UniqueMaterialIndex : UniqueMaterialIndices)
	{
		FMeshSectionInfo MeshSectionInfo(UniqueMaterialIndex);

		// enable/disable section collision according to settings
		MeshSectionInfo.bEnableCollision = Data->MergeData->InProxySettings.bCreateCollision;

		StaticMesh->GetSectionInfoMap().Set(0, SectionIndex, MeshSectionInfo);
		SectionIndex++;
	}

	if (ImposterBounds.IsValid)
	{
		const FBox StaticMeshBox = StaticMesh->GetBoundingBox();
		const FBox CombinedBox = StaticMeshBox + ImposterBounds;
		StaticMesh->SetPositiveBoundsExtension((CombinedBox.Max - StaticMeshBox.Max));
		StaticMesh->SetNegativeBoundsExtension((StaticMeshBox.Min - CombinedBox.Min));
		StaticMesh->CalculateExtendedBounds();
	}

	UStaticMesh::FBuildParameters BuildParameters;
	BuildParameters.bInSilent = true;
	BuildParameters.bInRebuildUVChannelData = true;
	BuildParameters.bInEnforceLightmapRestrictions = true;

	StaticMesh->Build(BuildParameters);

	OutAssetsToSync.Add(StaticMesh);

	if (OldStaticMesh != nullptr)
	{
		Algo::ForEach(RecreateRenderStateContext.GetComponentsUsingMesh(OldStaticMesh), [](UStaticMeshComponent* Component)
		{
			FObjectCacheEventSink::NotifyStaticMeshChanged_Concurrent(Component->GetStaticMeshComponentInterface());
		});
	}

	// Execute the delegate received from the user
	Data->MergeData->CallbackDelegate.ExecuteIfBound(JobGuid, OutAssetsToSync);
}
