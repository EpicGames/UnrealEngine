// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "MeshDescription.h"
#include "MaterialUtilities.h"

struct FMergeCompleteData;
struct FProxyGenerationData;
class FMeshMergeUtilities;

DECLARE_LOG_CATEGORY_EXTERN(LogMeshMerging, Verbose, All);

class FProxyGenerationProcessor : FTSTickerObjectBase
{
public:
	FProxyGenerationProcessor(const FMeshMergeUtilities* InOwner);
	~FProxyGenerationProcessor();

	void AddProxyJob(FGuid InJobGuid, FMergeCompleteData* InCompleteData);
	virtual bool Tick(float DeltaTime) override;
	void ProxyGenerationComplete(FMeshDescription& OutProxyMesh, struct FFlattenMaterial& OutMaterial, const FGuid OutJobGUID);

	//@third party BEGIN SIMPLYGON
	void ProxyGenerationFailed(const FGuid OutJobGUID, const FString& ErrorMessage);
	//@third party END SIMPLYGON

protected:
	/** Called when the map has changed*/
	void OnMapChange(uint32 MapFlags);

	/** Called when the current level has changed */
	void OnNewCurrentLevel();

	/** Clears the processing data array/map */
	void ClearProcessingData();

protected:
	/** Structure storing the data required during processing */
	struct FProxyGenerationData
	{
		FMeshDescription RawMesh;
		FFlattenMaterial Material;
		FMergeCompleteData* MergeData;
	};

	void ProcessJob(const FGuid& JobGuid, FProxyGenerationData* Data);
	void OnModuleConfigurationCompleted();

protected:
	/** Holds Proxy mesh job data together with the job Guid */
	TMap<FGuid, FMergeCompleteData*> ProxyMeshJobs;
	/** Holds Proxy generation data together with the job Guid */
	TMap<FGuid, FProxyGenerationData*> ToProcessJobDataMap;
	/** Critical section to keep ProxyMeshJobs/ToProcessJobDataMap access thread-safe */
	FCriticalSection StateLock;

	const FMeshMergeUtilities* Owner;
};

