// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshAdapter.h"

#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "MeshMergeHelpers.h"
#include "MeshUtilities.h"
#include "Modules/ModuleManager.h"
#include "Rendering/SkeletalMeshModel.h"
#include "UObject/Package.h"

FSkeletalMeshComponentAdapter::FSkeletalMeshComponentAdapter(USkeletalMeshComponent* InSkeletalMeshComponent)
	: SkeletalMeshComponent(InSkeletalMeshComponent), SkeletalMesh(InSkeletalMeshComponent->GetSkeletalMeshAsset())
{
	checkf(SkeletalMesh != nullptr, TEXT("Invalid skeletal mesh in adapter"));
	NumLODs = SkeletalMesh->GetLODNum();
}

int32 FSkeletalMeshComponentAdapter::GetNumberOfLODs() const
{
	return NumLODs;
}

void FSkeletalMeshComponentAdapter::RetrieveRawMeshData(int32 LODIndex, FMeshDescription& InOutRawMesh, bool bPropogateMeshData) const
{
	FMeshMergeHelpers::RetrieveMesh(SkeletalMeshComponent, LODIndex, InOutRawMesh, bPropogateMeshData);
}

void FSkeletalMeshComponentAdapter::RetrieveMeshSections(int32 LODIndex, TArray<FSectionInfo>& InOutSectionInfo) const
{
	FMeshMergeHelpers::ExtractSections(SkeletalMeshComponent, LODIndex, InOutSectionInfo);
}

int32 FSkeletalMeshComponentAdapter::GetMaterialIndex(int32 LODIndex, int32 SectionIndex) const
{
	const FSkeletalMeshLODInfo* LODInfoPtr = SkeletalMesh->GetLODInfo(LODIndex);
	if (LODInfoPtr && LODInfoPtr->LODMaterialMap.IsValidIndex(SectionIndex) && LODInfoPtr->LODMaterialMap[SectionIndex] != INDEX_NONE)
	{
		return LODInfoPtr->LODMaterialMap[SectionIndex];
	}
	return SkeletalMesh->GetImportedModel()->LODModels[LODIndex].Sections[SectionIndex].MaterialIndex;
}

UPackage* FSkeletalMeshComponentAdapter::GetOuter() const
{
	return nullptr;
}

FString FSkeletalMeshComponentAdapter::GetBaseName() const
{
	return SkeletalMesh->GetOutermost()->GetName();
}

FName FSkeletalMeshComponentAdapter::GetMaterialSlotName(int32 MaterialIndex) const
{
	return SkeletalMesh->GetMaterials()[MaterialIndex].MaterialSlotName;
}

FName FSkeletalMeshComponentAdapter::GetImportedMaterialSlotName(int32 MaterialIndex) const
{
	return SkeletalMesh->GetMaterials()[MaterialIndex].ImportedMaterialSlotName;
}

void FSkeletalMeshComponentAdapter::SetMaterial(int32 MaterialIndex, UMaterialInterface* Material)
{
	//We need to preserve the original material slot data
	const FSkeletalMaterial& OriginalMaterialSlot = SkeletalMesh->GetMaterials()[MaterialIndex];
	SkeletalMesh->GetMaterials()[MaterialIndex] = FSkeletalMaterial(Material, true, false, OriginalMaterialSlot.MaterialSlotName, OriginalMaterialSlot.ImportedMaterialSlotName);
}

void FSkeletalMeshComponentAdapter::RemapMaterialIndex(int32 LODIndex, int32 SectionIndex, int32 NewMaterialIndex)
{
	FSkeletalMeshLODInfo* LODInfoPtr = SkeletalMesh->GetLODInfo(LODIndex);
	check(LODInfoPtr);
	if (SkeletalMesh->GetImportedModel()->LODModels[LODIndex].Sections[SectionIndex].MaterialIndex == NewMaterialIndex)
	{
		if (LODInfoPtr->LODMaterialMap.IsValidIndex(SectionIndex))
		{
			LODInfoPtr->LODMaterialMap[SectionIndex] = INDEX_NONE;
		}
	}
	
	while (!LODInfoPtr->LODMaterialMap.IsValidIndex(SectionIndex))
	{
		LODInfoPtr->LODMaterialMap.Add(INDEX_NONE);
	}
	LODInfoPtr->LODMaterialMap[SectionIndex] = NewMaterialIndex;
}

int32 FSkeletalMeshComponentAdapter::AddMaterial(UMaterialInterface* Material)
{
	const int32 Index = SkeletalMesh->GetMaterials().Emplace(Material);
	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
	MeshUtilities.FixupMaterialSlotNames(SkeletalMesh);
	return Index;
}

int32 FSkeletalMeshComponentAdapter::AddMaterial(UMaterialInterface* Material, const FName& SlotName, const FName& ImportedSlotName)
{
	const int32 Index = SkeletalMesh->GetMaterials().Emplace(Material, true, false, SlotName, ImportedSlotName);
	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
	MeshUtilities.FixupMaterialSlotNames(SkeletalMesh);
	return Index;
}

void FSkeletalMeshComponentAdapter::UpdateUVChannelData()
{
	SkeletalMesh->UpdateUVChannelData(false);
}

bool FSkeletalMeshComponentAdapter::IsAsset() const
{
	return true;
}

int32 FSkeletalMeshComponentAdapter::LightmapUVIndex() const
{
	return INDEX_NONE;
}

FBoxSphereBounds FSkeletalMeshComponentAdapter::GetBounds() const
{
	return SkeletalMesh->GetBounds();
}
