// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshAdapter.h"

#include "Engine/StaticMesh.h"
#include "MaterialBakingStructures.h"
#include "MeshMergeHelpers.h"
#include "MeshUtilities.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"

FStaticMeshAdapter::FStaticMeshAdapter(UStaticMesh* InStaticMesh)
	: StaticMesh(InStaticMesh)
{
	checkf(StaticMesh != nullptr, TEXT("Invalid static mesh in adapter"));
	NumLODs = StaticMesh->GetNumLODs();
}

int32 FStaticMeshAdapter::GetNumberOfLODs() const
{
	return NumLODs;
}

void FStaticMeshAdapter::RetrieveRawMeshData(int32 LODIndex, FMeshDescription& InOutRawMesh, bool bPropogateMeshData) const
{
	FMeshMergeHelpers::RetrieveMesh(StaticMesh, LODIndex, InOutRawMesh);
}

void FStaticMeshAdapter::RetrieveMeshSections(int32 LODIndex, TArray<FSectionInfo>& InOutSectionInfo) const
{
	FMeshMergeHelpers::ExtractSections(StaticMesh, LODIndex, InOutSectionInfo);
}

int32 FStaticMeshAdapter::GetMaterialIndex(int32 LODIndex, int32 SectionIndex) const
{
	return StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex).MaterialIndex;
}

void FStaticMeshAdapter::ApplySettings(int32 LODIndex, FMeshData& InOutMeshData) const
{
	InOutMeshData.LightMapIndex = StaticMesh->GetLightMapCoordinateIndex();
}

UPackage* FStaticMeshAdapter::GetOuter() const
{
	return nullptr;
}

FString FStaticMeshAdapter::GetBaseName() const
{
	return StaticMesh->GetOutermost()->GetName();
}

FName FStaticMeshAdapter::GetMaterialSlotName(int32 MaterialIndex) const
{
	return StaticMesh->GetStaticMaterials()[MaterialIndex].MaterialSlotName;
}

FName FStaticMeshAdapter::GetImportedMaterialSlotName(int32 MaterialIndex) const
{
	return StaticMesh->GetStaticMaterials()[MaterialIndex].ImportedMaterialSlotName;
}

void FStaticMeshAdapter::SetMaterial(int32 MaterialIndex, UMaterialInterface* Material)
{
	const FStaticMaterial& OriginalMaterialSlot = StaticMesh->GetStaticMaterials()[MaterialIndex];
	StaticMesh->GetStaticMaterials()[MaterialIndex] = FStaticMaterial(Material, OriginalMaterialSlot.MaterialSlotName, OriginalMaterialSlot.ImportedMaterialSlotName);
}

void FStaticMeshAdapter::RemapMaterialIndex(int32 LODIndex, int32 SectionIndex, int32 NewMaterialIndex)
{
	FMeshSectionInfo SectionInfo = StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex);
	SectionInfo.MaterialIndex = NewMaterialIndex;
	StaticMesh->GetSectionInfoMap().Set(LODIndex, SectionIndex, SectionInfo);
}

int32 FStaticMeshAdapter::AddMaterial(UMaterialInterface* Material)
{
	int32 Index = StaticMesh->GetStaticMaterials().Emplace(Material);
	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
	MeshUtilities.FixupMaterialSlotNames(StaticMesh);
	return Index;
}

int32 FStaticMeshAdapter::AddMaterial(UMaterialInterface* Material, const FName& SlotName, const FName& ImportedSlotName)
{
	int32 Index = StaticMesh->GetStaticMaterials().Emplace(Material, SlotName, ImportedSlotName);
	IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
	MeshUtilities.FixupMaterialSlotNames(StaticMesh);
	return Index;
}

void FStaticMeshAdapter::UpdateUVChannelData()
{
	StaticMesh->UpdateUVChannelData(false);
}

bool FStaticMeshAdapter::IsAsset() const
{
	return true;
}

int32 FStaticMeshAdapter::LightmapUVIndex() const
{
	return StaticMesh->GetLightMapCoordinateIndex();
}

FBoxSphereBounds FStaticMeshAdapter::GetBounds() const
{
	return StaticMesh->GetBounds();
}
