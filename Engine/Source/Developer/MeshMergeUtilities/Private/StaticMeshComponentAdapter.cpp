// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshComponentAdapter.h"
#include "MaterialBakingStructures.h"

#include "Engine/MapBuildDataRegistry.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "MeshMergeUtilities.h"
#include "MeshMergeHelpers.h"
#include "StaticMeshComponentLODInfo.h"
#include "UObject/Package.h"

FStaticMeshComponentAdapter::FStaticMeshComponentAdapter(UStaticMeshComponent* InStaticMeshComponent)
	: StaticMeshComponent(InStaticMeshComponent), StaticMesh(InStaticMeshComponent->GetStaticMesh())
{
	checkf(StaticMesh != nullptr, TEXT("Invalid static mesh in adapter"));
	NumLODs = StaticMesh->GetNumLODs();
}

int32 FStaticMeshComponentAdapter::GetNumberOfLODs() const
{
	return NumLODs;
}

void FStaticMeshComponentAdapter::RetrieveRawMeshData(int32 LODIndex, FMeshDescription& InOutRawMesh, bool bPropogateMeshData) const
{
	FMeshMergeHelpers::RetrieveMesh(StaticMeshComponent, LODIndex, InOutRawMesh, bPropogateMeshData);
}

void FStaticMeshComponentAdapter::RetrieveMeshSections(int32 LODIndex, TArray<FSectionInfo>& InOutSectionInfo) const
{
	FMeshMergeHelpers::ExtractSections(StaticMeshComponent, LODIndex, InOutSectionInfo);
}

int32 FStaticMeshComponentAdapter::GetMaterialIndex(int32 LODIndex, int32 SectionIndex) const
{
	return StaticMesh->GetSectionInfoMap().Get(LODIndex, SectionIndex).MaterialIndex;
}

void FStaticMeshComponentAdapter::ApplySettings(int32 LODIndex, FMeshData& InOutMeshData) const
{
	if (StaticMeshComponent->LODData.IsValidIndex(LODIndex))
	{
		// Retrieve lightmap reference from the static mesh component (if it exists)
		const FStaticMeshComponentLODInfo& ComponentLODInfo = StaticMeshComponent->LODData[LODIndex];
		const FMeshMapBuildData* MeshMapBuildData = StaticMeshComponent->GetMeshMapBuildData(ComponentLODInfo);
		if (MeshMapBuildData)
		{
			InOutMeshData.LightMap = MeshMapBuildData->LightMap;
			InOutMeshData.LightMapIndex = StaticMeshComponent->GetStaticMesh()->GetLightMapCoordinateIndex();
			InOutMeshData.LightmapResourceCluster = MeshMapBuildData->ResourceCluster;
		}
	}

	if (!StaticMeshComponent->GetCustomPrimitiveData().Data.IsEmpty())
	{
		if (!InOutMeshData.PrimitiveData.IsSet())
		{
			InOutMeshData.PrimitiveData = FPrimitiveData();
		}

		InOutMeshData.PrimitiveData->CustomPrimitiveData = &StaticMeshComponent->GetCustomPrimitiveData();
	}
}

UPackage* FStaticMeshComponentAdapter::GetOuter() const
{
	return nullptr;
}

FString FStaticMeshComponentAdapter::GetBaseName() const
{
	return StaticMesh->GetOutermost()->GetName();
}

FName FStaticMeshComponentAdapter::GetMaterialSlotName(int32 MaterialIndex) const
{
	return StaticMeshComponent->GetMaterialSlotNames()[MaterialIndex];
}

FName FStaticMeshComponentAdapter::GetImportedMaterialSlotName(int32 MaterialIndex) const
{
	return FName();
}

void FStaticMeshComponentAdapter::SetMaterial(int32 MaterialIndex, UMaterialInterface* Material)
{
	StaticMeshComponent->SetMaterial(MaterialIndex, Material);
}

void FStaticMeshComponentAdapter::RemapMaterialIndex(int32 LODIndex, int32 SectionIndex, int32 NewMaterialIndex)
{
}

int32 FStaticMeshComponentAdapter::AddMaterial(UMaterialInterface* Material)
{
	return INDEX_NONE;
}

int32 FStaticMeshComponentAdapter::AddMaterial(UMaterialInterface* Material, const FName& SlotName, const FName& ImportedSlotName)
{
	return INDEX_NONE;
}

void FStaticMeshComponentAdapter::UpdateUVChannelData()
{
	StaticMesh->UpdateUVChannelData(false);
}

bool FStaticMeshComponentAdapter::IsAsset() const
{
	return false;
}

int32 FStaticMeshComponentAdapter::LightmapUVIndex() const
{
	return StaticMesh->GetLightMapCoordinateIndex();
}

FBoxSphereBounds FStaticMeshComponentAdapter::GetBounds() const
{
	return StaticMeshComponent->Bounds;
}
