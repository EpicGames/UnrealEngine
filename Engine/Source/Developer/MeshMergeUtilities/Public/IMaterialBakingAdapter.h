// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/BoxSphereBounds.h"

struct FMeshDescription;
struct FSectionInfo;
struct FMeshData;
class UMaterialInterface;

class IMaterialBakingAdapter
{
public:
	/** Returns the number of LODs for the data the adapter represents */
	virtual int32 GetNumberOfLODs() const = 0;
	/** Retrieves model data in FMeshDescription form */
	virtual void RetrieveRawMeshData(int32 LODIndex, FMeshDescription& InOutRawMesh, bool bPropagateMeshData) const = 0;
	/** Retrieves all mesh sections from underlying data */
	virtual void RetrieveMeshSections(int32 LODIndex, TArray<FSectionInfo>& InOutSectionInfo) const = 0;
	/** Returns lightmap UV index used by mesh data */
	virtual int32 LightmapUVIndex() const = 0;

	/** Sets material index to specified material value */
	virtual void SetMaterial(int32 MaterialIndex, UMaterialInterface* Material) = 0;
	/** Returns the material index for the given lod and section index */
	virtual int32 GetMaterialIndex(int32 LODIndex, int32 SectionIndex) const = 0;
	/** Remaps the material index for the given lod and section index to the specified new one */
	virtual void RemapMaterialIndex(int32 LODIndex, int32 SectionIndex, int32 NewMaterialIndex) = 0;
	/** Returns the material slot name for the given index */
	virtual FName GetMaterialSlotName(int32 MaterialIndex) const = 0;
	/** Returns the importer material slot name for the given index */
	virtual FName GetImportedMaterialSlotName(int32 MaterialIndex) const = 0;
	/** Adds a new material to the underlying asset/data */
	virtual int32 AddMaterial(UMaterialInterface* Material) = 0;
	/** Adds a new material to the underlying asset/data and specify the slot names */
	virtual int32 AddMaterial(UMaterialInterface* Material, const FName& SlotName, const FName& ImportedSlotName) = 0;

	/** Update UV channel data on object the adapter represents */
	virtual void UpdateUVChannelData() = 0;

	/** Ability to apply custom settings to the FMeshData structure */
	virtual void ApplySettings(int32 LODIndex, FMeshData& InOutMeshData) const = 0;

	/** Returns bounds of underlying data */
	virtual FBoxSphereBounds GetBounds() const = 0;
	/** Returns outer package to use when creating new assets */
	virtual UPackage* GetOuter() const = 0;
	/** Returns base name to use for newly created assets */
	virtual FString GetBaseName() const = 0;
	/** Returns whether or not the underlying data is an UAsset */
	virtual bool IsAsset() const = 0;
};