// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

struct FMeshDescription;
class UMaterialInterface;
class UStaticMeshComponent;

class IMeshMergeExtension
{
public:
	virtual void OnCreatedMergedRawMeshes(const TArray<UStaticMeshComponent*>& MergedComponents, const class FMeshMergeDataTracker& DataTracker, TArray<FMeshDescription>& MergedMeshLODs) = 0;

	virtual void OnCreatedProxyMaterial(const TArray<UStaticMeshComponent*>& MergedComponents, UMaterialInterface* ProxyMaterial) = 0;
};