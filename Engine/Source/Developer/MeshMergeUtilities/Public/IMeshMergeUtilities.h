// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Map.h"
#include "Containers/Array.h"
#include "IMeshReductionInterfaces.h"

struct FRawMeshExt;
struct FStaticMeshLODResources;
struct FBakeOutput;
struct FKAggregateGeom;
struct FMaterialData;
struct FMeshInstancingSettings;
struct FMeshMergingSettings;
struct FMeshProxySettings;
struct FFlattenMaterial;
class ALandscapeProxy;
class AActor;
class FProxyGenerationProcessor;
class UMaterialOptions;
class IMaterialBakingAdapter;
class IMeshMergeExtension;
class UBodySetup;
class UMaterialInterface;
class UStaticMeshComponent;
class USkeletalMeshComponent;
class UStaticMesh;
class UPackage;
class UWorld;
class FString;
enum class EFlattenMaterialProperties : uint8;

// Typedef pair of FVector2Ds, key == uv position offset and value == uv scaling factor
typedef TPair<FVector2D, FVector2D> FUVOffsetScalePair;

class IMeshMergeUtilities
{
public:
	virtual ~IMeshMergeUtilities() = default;

	/** Bakes out (in place) materials for given adapter */
	virtual void BakeMaterialsForComponent(TArray<TWeakObjectPtr<UObject>>& OptionObjects, IMaterialBakingAdapter* Adapter) const = 0;
	/** Bakes out (in place) materials for given static mesh component */
	virtual void BakeMaterialsForComponent(UStaticMeshComponent* StaticMeshComponent) const = 0;
	/** Bakes out (in place) materials for given skeletal mesh component */
	virtual void BakeMaterialsForComponent(USkeletalMeshComponent* SkeletalMeshComponent) const = 0;
	/** Bakes out (in place) materials for given static mesh asset */
	virtual void BakeMaterialsForMesh(UStaticMesh* Mesh) const = 0;

	/**
	* Merges components into a single Static Mesh (with possible baked out atlas-material)
	*
	* @param ComponentsToMerge - Components to merge
	* @param World - World in which the component reside
	* @param InSettings	- Settings to use
	* @param InBaseMaterial - Base Material used for final proxy material (optional, can be null)
	* @param InOuter - Outer if required
	* @param InBasePackageName - Destination package name for a generated assets. Used if Outer is null.
	* @param OutAssetsToSync Merged mesh assets
	* @param OutMergedActorLocation	World position of merged mesh
	* @param Screensize - final targeted screensize of mesh
	* @param bSilent Non-verbose flag
	* @return void
	*/
	virtual void MergeComponentsToStaticMesh(const TArray<UPrimitiveComponent*>& ComponentsToMerge, UWorld* World, const FMeshMergingSettings& InSettings, UMaterialInterface* InBaseMaterial, UPackage* InOuter, const FString& InBasePackageName, TArray<UObject*>& OutAssetsToSync, FVector& OutMergedActorLocation, const float ScreenSize, bool bSilent /*= false*/) const = 0;

	/**
	* Merges static mesh components into a single actor with multiple instanced static mesh components
	*
	* @param ComponentsToMerge - Components to merge
	* @param World - World in which the components reside
	* @level Level - The level in which to create the new merged actor
	* @param InSettings	- Settings to use
	* @param bActuallyMerge Whether to actually perform the merge (could just be fetching predicted results)
	* @param OutResultsText (optional) Results of the merge
	* @return void
	*/
	virtual void MergeComponentsToInstances(const TArray<UPrimitiveComponent*>& ComponentsToMerge, UWorld* World, ULevel* Level, const FMeshInstancingSettings& InSettings, bool bActuallyMerge = true, bool bReplaceSourceActors = false, FText* OutResultsText = nullptr) const = 0;

	/**
	*	Merges list of actors into single proxy mesh
	*
	*	@param	Actors					List of Actors to merge
	*	@param	InProxySettings			Merge settings
	*	@param	InOuter					Package for a generated assets, if NULL new packages will be created for each asset
	*	@param	ProxyBasePackageName	Will be used for naming generated assets, in case InOuter is not specified ProxyBasePackageName will be used as long package name for creating new packages
	*	@param	OutAssetsToSync			Result assets - mesh, material
	*/	
	virtual void CreateProxyMesh(const TArray<AActor*>& InActors, const struct FMeshProxySettings& InMeshProxySettings, UPackage* InOuter, const FString& InProxyBasePackageName, const FGuid InGuid, const FCreateProxyDelegate& InProxyCreatedDelegate, const bool bAllowAsync = false, const float ScreenSize = 1.0f) const = 0;

	/**
	*	Merges list of actors into single proxy mesh
	*
	*	@param	Actors					List of Actors to merge
	*	@param	InProxySettings			Merge settings
	*	@param	InBaseMaterial			Base Material used for final proxy material
	*	@param	InOuter					Package for a generated assets, if NULL new packages will be created for each asset
	*	@param	ProxyBasePackageName	Will be used for naming generated assets, in case InOuter is not specified ProxyBasePackageName will be used as long package name for creating new packages
	*	@param	OutAssetsToSync			Result assets - mesh, material
	*/
	virtual void CreateProxyMesh(const TArray<AActor*>& InActors, const struct FMeshProxySettings& InMeshProxySettings, UMaterialInterface* InBaseMaterial, UPackage* InOuter, const FString& InProxyBasePackageName, const FGuid InGuid, const FCreateProxyDelegate& InProxyCreatedDelegate, const bool bAllowAsync = false, const float ScreenSize = 1.0f) const = 0;

	/**
	*	Merges list of static meshes into single proxy mesh
	*
	*	@param	InComponents		    List of meshes to merge
	*	@param	InProxySettings			Merge settings
	*	@param	InOuter					Package for a generated assets, if NULL new packages will be created for each asset
	*	@param	ProxyBasePackageName	Will be used for naming generated assets, in case InOuter is not specified ProxyBasePackageName will be used as long package name for creating new packages
	*	@param	OutAssetsToSync			Result assets - mesh, material
	*/
	virtual void CreateProxyMesh(const TArray<UStaticMeshComponent*>& InComponents, const struct FMeshProxySettings& InMeshProxySettings, UPackage* InOuter, const FString& InProxyBasePackageName, const FGuid InGuid, const FCreateProxyDelegate& InProxyCreatedDelegate, const bool bAllowAsync = false, const float ScreenSize = 1.0f) const = 0;

	/**
	*	Merges list of static meshes into single proxy mesh
	*
	*	@param	InComponents		    List of meshes to merge
	*	@param	InProxySettings			Merge settings
	*	@param	InBaseMaterial			Base Material used for final proxy material
	*	@param	InOuter					Package for a generated assets, if NULL new packages will be created for each asset
	*	@param	ProxyBasePackageName	Will be used for naming generated assets, in case InOuter is not specified ProxyBasePackageName will be used as long package name for creating new packages
	*	@param	OutAssetsToSync			Result assets - mesh, material
	*/
	virtual void CreateProxyMesh(const TArray<UStaticMeshComponent*>& InComponents, const struct FMeshProxySettings& InMeshProxySettings, UMaterialInterface* InBaseMaterial, UPackage* InOuter, const FString& InProxyBasePackageName, const FGuid InGuid, const FCreateProxyDelegate& InProxyCreatedDelegate, const bool bAllowAsync = false, const float ScreenSize = 1.0f) const = 0;

	/** Extract mesh data in FMeshDescription form from the provided static mesh component */
	virtual void RetrieveMeshDescription(const UStaticMeshComponent* InStaticMeshComponent, int32 LODIndex, FMeshDescription& InOutMeshDescription, bool bPropagateVertexColours) const = 0;
	/** Extract mesh data in FMeshDescription form from the provided skeletal mesh component */
	virtual void RetrieveMeshDescription(const USkeletalMeshComponent* InSkeletalMeshComponent, int32 LODIndex, FMeshDescription& InOutMeshDescription, bool bPropagateVertexColours) const = 0;
	/** Extract mesh data in FMeshDescription form from the provided static mesh */
	virtual void RetrieveMeshDescription(const UStaticMesh* InStaticMesh, int32 LODIndex, FMeshDescription& InOutMeshDescription) const = 0;

	/**
	 * Extract physics geometry and body setup from set of primitive components.
	 * 
	 * @param InComponents			List of components to extract the physics geometry from.
	 * @param InOutPhysicsGeometry	The list of extracted physics geometry.
	 * @param OutBodySetupSource	The BodySetup extracted.
	 */
	virtual void RetrievePhysicsData(const TArray<UPrimitiveComponent*>& InComponents, TArray<FKAggregateGeom>& InOutPhysicsGeometry, UBodySetup*& OutBodySetupSource) const = 0;

	virtual void RegisterExtension(IMeshMergeExtension* InExtension) = 0;
	virtual void UnregisterExtension(IMeshMergeExtension* InExtension) = 0;
};
