// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/StaticMesh.h"
#include "MeshDescription.h"
#include "PhysicsEngine/AggregateGeom.h"
#include "StaticMeshResources.h"

#include "LightMap.h"
#include "ShadowMap.h"
#include "Engine/MapBuildDataRegistry.h"

/** Structure holding intermediate mesh merging data that is used throughout the mesh merging and proxy creation processes */
struct FMeshMergeData
{
	FMeshMergeData()
		: RawMesh(nullptr)
		, SourceStaticMesh(nullptr)
		, bIsClippingMesh(false)
	{}

	void ReleaseData()
	{
		if (RawMesh != nullptr)
		{
			delete RawMesh;
			RawMesh = nullptr;
		}
	}
	
	/** Raw mesh data from the source static mesh */
	FMeshDescription* RawMesh;	

	/** Contains the original texture bounds, if the material requires baking down per-vertex data */
	TArray<FBox2D> TexCoordBounds;
	/** Will contain non-overlapping texture coordinates, if the material requires baking down per-vertex data */	
	TArray<FVector2D> NewUVs;
	/** Pointer to the source static mesh instance */
	class UStaticMesh* SourceStaticMesh;
	/** If set, the raw mesh should be used as clipping geometry */
	bool bIsClippingMesh;
	FString DebugJobName;
};

struct FInstancedMeshMergeData : public FMeshMergeData
{
	TArray<FTransform> InstanceTransforms;
};

/** Structure for encapsulating per LOD mesh merging data */
struct FRawMeshExt
{
	FRawMeshExt()
		: SourceStaticMesh(nullptr)
	{
		FMemory::Memzero(bShouldExportLOD);
	}

	FMeshMergeData MeshLODData[MAX_STATIC_MESH_LODS];
	FKAggregateGeom	AggGeom;

	/** Pointer to the source static mesh instance */
	class UStaticMesh* SourceStaticMesh;	

	class UStaticMeshComponent* Component;

	/** Specific LOD index that is being exported */
	int32 ExportLODIndex;

	FLightMapRef LightMap;
	FShadowMapRef ShadowMap;

	/** Whether or not a specific LOD should be exported */
	bool bShouldExportLOD[MAX_STATIC_MESH_LODS];
	/** Max LOD index that is exported */
	int32 MaxLODExport;
};
