// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StaticMeshResources.h"
#include "Materials/MaterialInstance.h"
#include "MaterialUtilities.h"
#include "StaticMeshAttributes.h"

#define UE_API MESHMERGEUTILITIES_API

typedef TPair<uint32, uint32> MeshLODPair;
typedef TPair<uint32, uint32> SectionRemapPair;
typedef TPair<uint32, uint32> MaterialRemapPair;

struct FSectionInfo;

/** Structure representing a mesh and lod index */
struct FMeshLODKey
{
public:
	FMeshLODKey(int32 MeshIndex, int32 LODIndex, UStaticMesh* InMesh = nullptr)
	{
		CombinedKey = ((LODIndex & 0xffff) << 16) | (MeshIndex & 0xffff);
		Mesh = InMesh;
		VertexColorHash = 0;
	}

	explicit FMeshLODKey(uint32 InCombinedKey)
	{
		CombinedKey = InCombinedKey;
		Mesh = nullptr;
		VertexColorHash = 0;
	}

	UStaticMesh* GetMesh() const
	{
		return Mesh;
	}

	int32 GetMeshIndex() const
	{
		return (CombinedKey & 0x0000ffff);
	}

	int32 GetLODIndex() const
	{
		return (CombinedKey & 0xffff0000) >> 16;
	}

	uint32 GetVertexColorHash() const
	{
		return VertexColorHash;
	}

	void SetVertexColorHash(uint32 InVertexColorHash)
	{
		VertexColorHash = InVertexColorHash;
	}

	inline bool operator==(const FMeshLODKey& Other) const
	{
		return CombinedKey == Other.CombinedKey;
	}

	inline friend uint32 GetTypeHash(const FMeshLODKey& Item)
	{
		return Item.CombinedKey;
	}

protected:
	uint32 CombinedKey;
	UStaticMesh* Mesh;
	uint32 VertexColorHash;
};

/** This helps identify materials that can be collapsed because of equivalence */
struct FMaterialKey
{
	FMaterialKey(const UMaterialInterface* InMaterial)
		: Material(InMaterial)
	{
	}

	inline bool operator==(const FMaterialKey& Other) const
	{
		// Perform an optional custom comparison if we are trying to collapse material instances
		const UMaterialInstance* MaterialInstance0 = Cast<UMaterialInstance>(Material);
		const UMaterialInstance* MaterialInstance1 = Cast<UMaterialInstance>(Other.Material);
		if(MaterialInstance0 && MaterialInstance1)
		{
			return MaterialInstance0->Equivalent(MaterialInstance1);
		}
		else
		{
			return Material == Other.Material;
		}
	}

	const UMaterialInterface* Material;
};

/** Typedefs to allow for some nicer looking loops */
typedef TMap<FMeshLODKey, FMeshDescription>::TConstIterator TConstRawMeshIterator;
typedef TMap<FMeshLODKey, FMeshDescription>::TIterator TRawMeshIterator;

typedef TArray<int32>::TConstIterator TConstLODIndexIterator;

/** Used to keep track of in-flight data while meshes are merged and their corresponding materials baked down */
class FMeshMergeDataTracker
{	
public:
	UE_API FMeshMergeDataTracker();

	/** Looks at all available raw mesh data and processes it to populate some flags */
	UE_API void ProcessRawMeshes();

	/** Adding a mapping between the index for an original mesh section index and the mesh section it will be indexed to in the final mesh */
	UE_API void AddSectionRemapping(int32 MeshIndex, int32 LODIndex, int32 OriginalIndex, int32 UniqueIndex);
	/** Retrieves the MeshLOD keys from which the original sections are mapped to the unique section index */
	UE_API void GetMeshLODsMappedToUniqueSection(int32 UniqueIndex, TArray<FMeshLODKey>& InOutMeshLODs);
	/** Retrieves all section mappings for the MeshLOD key*/
	UE_API void GetMappingsForMeshLOD(FMeshLODKey Key, TArray<SectionRemapPair>& InOutMappings);
	
	/** Adds or retrieves raw mesh data for the mesh and LOD index */
	UE_API FMeshDescription& AddAndRetrieveRawMesh(int32 MeshIndex, int32 LODIndex, UStaticMesh* InMesh);
	/** Removes raw mesh entry for the given mesh and LOD index */
	UE_API void RemoveRawMesh(int32 MeshIndex, int32 LODIndex);
	/** Retrieves Raw Mesh ptr for the given mesh and LOD index */
	UE_API FMeshDescription* GetRawMeshPtr(int32 MeshIndex, int32 LODIndex);
	/** Retrieves Raw Mesh ptr for the given MeshLOD key */
	UE_API FMeshDescription* GetRawMeshPtr(FMeshLODKey Key);
	/** Tries to retrieve a FMeshDescription and returns the LOD index it found an entry for */
	UE_API FMeshDescription* FindRawMeshAndLODIndex(int32 MeshIndex, int32& OutLODIndex);
	/** Tries to retrieve a FMeshDescription for the given mesh and LOD index, if it can't it will try to find an entry for each LOD levle below InOutDesiredLODIndex */
	UE_API FMeshDescription* TryFindRawMeshForLOD(int32 MeshIndex, int32& InOutDesiredLODIndex);

	/** Returns a const key/value iterator for the FMeshDescription entries */
	UE_API TConstRawMeshIterator GetConstRawMeshIterator() const;
	/** Returns a non-const key/value iterator for the FMeshDescription entries */ 
	UE_API TRawMeshIterator GetRawMeshIterator();

	/** Adds a record of what channel lightmap data is stored at */
	UE_API void AddLightmapChannelRecord(int32 MeshIndex, int32 LODIndex, int32 LightmapChannelIndex);

	/** Adds (unique) section to stored data */
	UE_API int32 AddSection(const FSectionInfo& SectionInfo);
	/** Returns the number of unique sections */
	UE_API int32 NumberOfUniqueSections() const;
	/** Returns the material used by the unique section */
	UE_API UMaterialInterface* GetMaterialForSectionIndex(int32 SectionIndex);
	/** Returns the unique section instance */
	UE_API const FSectionInfo& GetSection(int32 SectionIndex) const;
	/** Clears out unique section to be replaced with the baked material one */
	UE_API void AddBakedMaterialSection(const FSectionInfo& SectionInfo);
	
	/** Add a material slot name for a unique material instance. */
	UE_API void AddMaterialSlotName(UMaterialInterface *MaterialInterface, FName MaterialSlotName);
	/** Get the material slot name from a unique material instance. */
	UE_API FName GetMaterialSlotName(UMaterialInterface *MaterialInterface) const;

	/** Adds a LOD index which will be part of the final merged mesh */
	UE_API void AddLODIndex(int32 LODIndex);
	/** Retrieves number of LODs part of the final merged mesh */
	UE_API int32 GetNumLODsForMergedMesh() const;
	/** Iterates over LOD indices for mesh */
	UE_API TConstLODIndexIterator GetLODIndexIterator() const;

	/** Add number of lightmap pixels used for one of the Meshes */
	UE_API void AddLightMapPixels(int32 Pixels);
	/** Returns the texture dimension required to distribute all of the lightmap pixels */
	UE_API int32 GetLightMapDimension() const;

	/** Returns whether or not any raw mesh entry contains vertex colors for the specified LOD index */
	UE_API bool DoesLODContainVertexColors(int32 LODIndex) const;
	/** Returns whether or not any raw mesh entry contains vertex colors for any LOD index */
	UE_API bool DoesAnyLODContainVertexColors() const;
	/** Returns whether or not any raw mesh entry contains texture coordinates for the specified UV channel and LOD index */
	UE_API bool DoesUVChannelContainData(int32 UVChannel, int32 LODIndex) const;
	/** Returns whether or not any raw mesh entry contains texture coordinates for the specified UV channel and any LOD index */
	UE_API bool DoesUVChannelContainData(int32 UVChannel) const;
	/** Returns whether or not the raw mesh entry for the given MeshLOD key requires unique UVs for baking out its material(s) */
	UE_API bool DoesMeshLODRequireUniqueUVs(FMeshLODKey Key);
	/** Returns the first available UV channel across all raw mesh entries, which will be a good fit for the lightmap UV index in the final mesh */
	UE_API int32 GetAvailableLightMapUVChannel() const;

	UE_API void AddComponentToWedgeMapping(int32 MeshIndex, int32 LODIndex, uint32 WedgeIndex);
	UE_API uint32 GetComponentToWedgeMappng(int32 MeshIndex, int32 LODIndex) const;

	UE_API double GetTextureSizeFromTargetTexelDensity(float InTargetTexelDensity) const;

protected:
	// Mesh / LOD index, RawMesh
	TMap<FMeshLODKey, FMeshDescription> RawMeshLODs;

	// Mesh / LOD index, lightmap channel
	TMap<FMeshLODKey, int32> LightmapChannelLODs;

	// Whether a key requires unique UVs 
	TArray<FMeshLODKey> RequiresUniqueUVs;

	//Use this map to recycle the material slot name
	TMap<UMaterialInterface *, FName> MaterialInterfaceToMaterialSlotName;
	
	/** Flags for UV and vertex color usage */
	bool bWithVertexColors[MAX_STATIC_MESH_LODS];
	bool bOcuppiedUVChannels[MAX_STATIC_MESH_LODS][MAX_MESH_TEXTURE_COORDS_MD];
	/** First available UV channel across all RawMesh entries */
	int32 AvailableLightMapUVChannel;
	int32 SummedLightMapPixels;

	/** Remapping pairs for each mesh and LOD index combination */
	TMultiMap<FMeshLODKey, SectionRemapPair> UniqueSectionIndexPerLOD;
	/** Maps from each unique section index to all the RawMesh entries which contain an original section that's mapped to it */
	TMultiMap<uint32, FMeshLODKey> UniqueSectionToMeshLOD;

	/** All LOD indices which should be populated in the final merged mesh */
	TArray<int32> LODIndices;
	
	/** Unique set of sections in mesh */
	TArray<FSectionInfo> UniqueSections;

	TMap<FMeshLODKey, uint32> ComponentToWedgeOffsets;
};

#undef UE_API
