// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMaterialBakingAdapter.h"

class UStaticMesh;

/** Adapter which takes a Static Mesh asset to use for material baking (allows for changes to the asset itself) */
class FStaticMeshAdapter : public IMaterialBakingAdapter
{
public:
	FStaticMeshAdapter(UStaticMesh* InStaticMesh);

	/** Begin IMaterialBakingAdapter overrides */
	virtual int32 GetNumberOfLODs() const override;
	virtual void RetrieveRawMeshData(int32 LODIndex, FMeshDescription& InOutRawMesh, bool bPropogateMeshData) const override;
	virtual void RetrieveMeshSections(int32 LODIndex, TArray<FSectionInfo>& InOutSectionInfo) const override;
	virtual int32 GetMaterialIndex(int32 LODIndex, int32 SectionIndex) const override;
	virtual void ApplySettings(int32 LODIndex, FMeshData& InOutMeshData) const override;
	virtual UPackage* GetOuter() const override;
	virtual FString GetBaseName() const override;
	virtual FName GetMaterialSlotName(int32 MaterialIndex) const override;
	virtual FName GetImportedMaterialSlotName(int32 MaterialIndex) const override;
	virtual void SetMaterial(int32 MaterialIndex, UMaterialInterface* Material) override;
	virtual void RemapMaterialIndex(int32 LODIndex, int32 SectionIndex, int32 NewMaterialIndex) override;
	virtual int32 AddMaterial(UMaterialInterface* Material) override;
	virtual int32 AddMaterial(UMaterialInterface* Material, const FName& SlotName, const FName& ImportedSlotName) override;
	virtual void UpdateUVChannelData() override;
	virtual bool IsAsset() const override;
	virtual int32 LightmapUVIndex() const override;
	virtual FBoxSphereBounds GetBounds() const override;
	/** End IMaterialBakingAdapter overrides */

protected:
	UStaticMesh* StaticMesh;
	int32 NumLODs;
};