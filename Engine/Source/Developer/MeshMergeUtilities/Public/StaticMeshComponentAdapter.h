// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMaterialBakingAdapter.h"

#define UE_API MESHMERGEUTILITIES_API

class UStaticMeshComponent;
class UStaticMesh;

/** Adapter which takes a Static Mesh Component instance to use for material baking (does not allow for changes to the underlying asset itself) */
class FStaticMeshComponentAdapter : public IMaterialBakingAdapter
{
public:
	UE_API FStaticMeshComponentAdapter(UStaticMeshComponent* InStaticMeshComponent);

	/** Begin IMaterialBakingAdapter overrides */
	UE_API virtual int32 GetNumberOfLODs() const override;
	UE_API virtual void RetrieveRawMeshData(int32 LODIndex, FMeshDescription& InOutRawMesh, bool bPropogateMeshData) const override;
	UE_API virtual void RetrieveMeshSections(int32 LODIndex, TArray<FSectionInfo>& InOutSectionInfo) const override;
	UE_API virtual int32 GetMaterialIndex(int32 LODIndex, int32 SectionIndex) const override;	
	UE_API virtual void ApplySettings(int32 LODIndex, FMeshData& InOutMeshData) const override;
	UE_API virtual UPackage* GetOuter() const override;
	UE_API virtual FString GetBaseName() const override;
	UE_API virtual FName GetMaterialSlotName(int32 MaterialIndex) const override;
	UE_API virtual FName GetImportedMaterialSlotName(int32 MaterialIndex) const override;
	UE_API virtual void SetMaterial(int32 MaterialIndex, UMaterialInterface* Material) override;
	UE_API virtual void RemapMaterialIndex(int32 LODIndex, int32 SectionIndex, int32 NewMaterialIndex) override;
	UE_API virtual int32 AddMaterial(UMaterialInterface* Material) override;
	UE_API virtual int32 AddMaterial(UMaterialInterface* Material, const FName& SlotName, const FName& ImportedSlotName) override;
	UE_API virtual void UpdateUVChannelData() override;
	UE_API virtual bool IsAsset() const override;
	UE_API virtual int32 LightmapUVIndex() const override;
	UE_API virtual FBoxSphereBounds GetBounds() const override;
	/** End IMaterialBakingAdapter overrides */

protected:
	UStaticMeshComponent* StaticMeshComponent;
	UStaticMesh* StaticMesh;
	int32 NumLODs;
};

#undef UE_API
